// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ClothPainter : ModuleRules
{
	public ClothPainter(ReadOnlyTargetRules Target) : base(Target)
    {
		PrivateDependencyModuleNames.AddRange(
			new string[] 
            {
                "Core",
                "CoreUObject",
                "Engine",
                "InputCore",
                "Slate",
                "SlateCore",
				"EditorFramework",
				"UnrealEd",
                "MainFrame",
                "PropertyEditor",
                "Kismet",
                "AssetTools",
                "ClassViewer",
                "SkeletalMeshEditor",                
                "AssetRegistry",
                "MeshPaint",
                "ClothingSystemRuntimeCommon",
                "ClothingSystemRuntimeNv",
                "ClothingSystemRuntimeInterface",
                "ClothingSystemEditorInterface",
                "SkeletalMeshEditor",
                "AdvancedPreviewScene",
				"ToolMenus",
				"EditorWidgets",
				"ToolWidgets",
			}
		);
    }
}
