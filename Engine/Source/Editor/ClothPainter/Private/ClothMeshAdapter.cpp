// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothMeshAdapter.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "ClothLODData.h"
#include "ClothPhysicalMeshData.h"
#include "ClothingAsset.h"
#include "ClothingAssetBase.h"
#include "Components/MeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Set.h"
#include "Engine/HitResult.h"
#include "Engine/NetSerialization.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/Plane.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IMeshPaintGeometryAdapter;
class UTexture;

TSharedPtr<IMeshPaintGeometryAdapter> FClothMeshPaintAdapterFactory::Construct(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex) const
{
	if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(InComponent))
	{
		if (Component->GetSkeletalMeshAsset() != nullptr)
		{
			TSharedRef<FClothMeshPaintAdapter> Result = MakeShareable(new FClothMeshPaintAdapter());
			if (Result->Construct(InComponent, InPaintingMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}

bool FClothMeshPaintAdapter::Construct(UMeshComponent* InComponent, int32 InPaintingClothLODIndex)
{
	SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent);
	if (SkeletalMeshComponent != nullptr)
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			ReferencedSkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
			PaintingClothLODIndex = InPaintingClothLODIndex;
			PaintingClothMaskIndex = INDEX_NONE;

			const bool bSuccess = Initialize();
			return bSuccess;
		}
	}

	return false;
}

bool FClothMeshPaintAdapter::Initialize()
{
	check(ReferencedSkeletalMesh == SkeletalMeshComponent->GetSkeletalMeshAsset());

	bool bHaveAsset = false;
	bool bBaseInit = false;

	UClothingAssetCommon* ClothingAsset = Cast<UClothingAssetCommon>(SelectedAsset);

	bHaveAsset = ClothingAsset && ClothingAsset->LodData.IsValidIndex(PaintingClothLODIndex);
	bBaseInit = FBaseMeshPaintGeometryAdapter::Initialize();

	return bHaveAsset && bBaseInit;
}

bool ClothSegmentTriangleIntersection(const FVector& StartPoint, const FVector& EndPoint, const FVector& A, const FVector& B, const FVector& C, FVector& OutIntersectPoint, FVector& OutTriangleNormal)
{
	const FVector BA = A - B;
	const FVector CB = B - C;
	const FVector TriNormal = BA ^ CB;

	bool bCollide = FMath::SegmentPlaneIntersection(StartPoint, EndPoint, FPlane(A, TriNormal), OutIntersectPoint);
	if (!bCollide)
	{
		return false;
	}

	FVector BaryCentric = FMath::ComputeBaryCentric2D(OutIntersectPoint, A, B, C);
	if (BaryCentric.X > 0.0f && BaryCentric.Y > 0.0f && BaryCentric.Z > 0.0f)
	{
		OutTriangleNormal = TriNormal;
		return true;
	}
	return false;
}
bool FClothMeshPaintAdapter::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	const int32 NumTriangles = MeshIndices.Num() / 3;
	float MinDistance = FLT_MAX;
	FVector Intersect;
	FVector Normal;
	
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
	{
		FVector IntersectPoint;
		FVector HitNormal;
		bool bHit = ClothSegmentTriangleIntersection(Start, End, MeshVertices[MeshIndices[(TriangleIndex * 3) + 0]], MeshVertices[MeshIndices[(TriangleIndex * 3) + 1]], MeshVertices[MeshIndices[(TriangleIndex * 3) + 2]], IntersectPoint, HitNormal);

		if (bHit)
		{
			const float Distance = (Start - IntersectPoint).SizeSquared();
			if (Distance < MinDistance)
			{
				MinDistance = Distance;
				Intersect = IntersectPoint;
				Normal = HitNormal;
			}
		}
	}

	if (MinDistance != FLT_MAX)
	{
		OutHit.Component = SkeletalMeshComponent;
		OutHit.Normal = Normal.GetSafeNormal();
		OutHit.Location = Intersect;
		OutHit.bBlockingHit = true;
		return true;
	}

	return false;
}

void FClothMeshPaintAdapter::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{

}

void FClothMeshPaintAdapter::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const
{

}

void FClothMeshPaintAdapter::PreEdit()
{
	ReferencedSkeletalMesh->Modify();
	SelectedAsset->Modify();
}

void FClothMeshPaintAdapter::PostEdit()
{
	
}

void FClothMeshPaintAdapter::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	checkf(false, TEXT("Not supported, should not be vertex painting cloth"));
}

void FClothMeshPaintAdapter::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	checkf(false, TEXT("Not supported, should not be vertex painting cloth"));
}

void FClothMeshPaintAdapter::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	checkf(false, TEXT("Not supported"));
}

FMatrix FClothMeshPaintAdapter::GetComponentToWorldMatrix() const
{
	return SkeletalMeshComponent->GetComponentToWorld().ToMatrixWithScale();
}

TArray<FVector> FClothMeshPaintAdapter::SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const
{
	// Get list of intersecting triangles with given sphere data
	const TArray<uint32> IntersectedTriangles = SphereIntersectTriangles(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, bOnlyFrontFacing);
	// Get a list of unique vertices indexed by the influenced triangles
	TSet<int32> InfluencedVertices;
	InfluencedVertices.Reserve(IntersectedTriangles.Num());
	for (int32 IntersectedTriangle : IntersectedTriangles)
	{
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 0]);
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 2]);
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 1]);
	}

	TArray<FVector> InRangeVertices;
	InRangeVertices.Empty(MeshVertices.Num());
	for (int32 VertexIndex : InfluencedVertices)
	{
		const FVector& Vertex = MeshVertices[VertexIndex];
		if (FVector::DistSquared(ComponentSpaceBrushPosition, Vertex) <= ComponentSpaceSquaredBrushRadius)
		{
			InRangeVertices.Add(Vertex);
		}
	}

	return InRangeVertices;
}

void FClothMeshPaintAdapter::SetSelectedClothingAsset(const FGuid& InAssetGuid, int32 InAssetLod, int32 InMaskIndex)
{
	SelectedAsset = nullptr;
	if(InAssetGuid.IsValid() && ReferencedSkeletalMesh)
	{
		for(UClothingAssetBase* Asset : ReferencedSkeletalMesh->GetMeshClothingAssets())
		{
			if(const UClothingAssetCommon* const ConcreteAsset = Cast<const UClothingAssetCommon>(Asset))
			{
				if(ConcreteAsset->GetAssetGuid() == InAssetGuid)
				{
					if(ConcreteAsset->IsValidLod(InAssetLod))
					{
						const FClothLODDataCommon& LodData = ConcreteAsset->LodData[InAssetLod];

						if(LodData.PointWeightMaps.IsValidIndex(InMaskIndex))
						{
							PaintingClothLODIndex = InAssetLod;
							PaintingClothMaskIndex = InMaskIndex;
							SelectedAsset = Asset;
						}
					}

					break;
				}
			}
		}
	}

	if(SelectedAsset)
	{
		Initialize();
	}
}

const TArray<int32>* FClothMeshPaintAdapter::GetVertexNeighbors(int32 InVertexIndex) const
{
	for(const FClothAssetInfo& Info : AssetInfoMap)
	{
		if(InVertexIndex >= Info.VertexStart && InVertexIndex < Info.VertexEnd)
		{
			return &Info.NeighborMap[InVertexIndex - Info.VertexStart];
		}
	}

	return nullptr;
}

bool FClothMeshPaintAdapter::InitializeVertexData()
{
	int32 VertexOffset = 0;
	int32 IndexOffset = 0;

	AssetInfoMap.Empty();

	MeshVertices.Empty();
	MeshIndices.Empty();

	if (UClothingAssetCommon* const ConcreteAsset = Cast<UClothingAssetCommon>(SelectedAsset))
	{
		if(const UDebugSkelMeshComponent* const DebugComponent = Cast<UDebugSkelMeshComponent>(SkeletalMeshComponent))
		{
			if(DebugComponent->SkinnedSelectedClothingPositions.Num() > 0)
			{
				if (ConcreteAsset->LodData.IsValidIndex(PaintingClothLODIndex))
				{
					const FClothLODDataCommon& LODData = ConcreteAsset->LodData[PaintingClothLODIndex];
					const FClothPhysicalMeshData& MeshData = LODData.PhysicalMeshData;

					MeshVertices.Append(DebugComponent->SkinnedSelectedClothingPositions);
					MeshIndices.Append(MeshData.Indices);

					for(int32 Index = IndexOffset; Index < MeshIndices.Num(); ++Index)
					{
						MeshIndices[Index] += VertexOffset;
					}

					FClothAssetInfo Info;
					Info.IndexStart = IndexOffset;
					Info.VertexStart = VertexOffset;

					IndexOffset += MeshData.Indices.Num();
					VertexOffset += MeshData.Vertices.Num();

					Info.IndexEnd = IndexOffset;
					Info.VertexEnd = VertexOffset;
					Info.Asset = ConcreteAsset;

					// Set up the edge map / neighbor map

					// Pre fill the map, 1 per index
					Info.NeighborMap.AddDefaulted(Info.VertexEnd - Info.VertexStart);

					// Fill in neighbors defined by triangles
					const int32 NumTris = MeshIndices.Num() / 3;
					for(int32 TriIndex = 0; TriIndex < NumTris; ++TriIndex)
					{
						const int32 I0 = MeshIndices[TriIndex * 3];
						const int32 I1 = MeshIndices[TriIndex * 3 + 1];
						const int32 I2 = MeshIndices[TriIndex * 3 + 2];

						TArray<int32>& I0Neighbors = Info.NeighborMap[I0];
						TArray<int32>& I1Neighbors = Info.NeighborMap[I1];
						TArray<int32>& I2Neighbors = Info.NeighborMap[I2];

						I0Neighbors.AddUnique(I1);
						I0Neighbors.AddUnique(I2);

						I1Neighbors.AddUnique(I0);
						I1Neighbors.AddUnique(I2);

						I2Neighbors.AddUnique(I0);
						I2Neighbors.AddUnique(I1);
					}

					AssetInfoMap.Add(Info);
				}
			}
		}
	}

	return true;
}

FPointWeightMap* FClothMeshPaintAdapter::GetCurrentMask() const
{
	if(HasValidSelection())
	{
		if(UClothingAssetCommon* const ConcreteAsset = Cast<UClothingAssetCommon>(SelectedAsset))
		{
			return &ConcreteAsset->LodData[PaintingClothLODIndex].PointWeightMaps[PaintingClothMaskIndex];
		}
	}

	return nullptr;
}

bool FClothMeshPaintAdapter::HasValidSelection() const
{
	if(const UClothingAssetCommon* const ConcreteAsset = Cast<UClothingAssetCommon>(SelectedAsset))
	{
		// Only valid selection if we have a valid asset, asset LOD and a mask
		if(ConcreteAsset->LodData.IsValidIndex(PaintingClothLODIndex) &&
			ConcreteAsset->LodData[PaintingClothLODIndex].PointWeightMaps.IsValidIndex(PaintingClothMaskIndex))
		{
			return true;
		}
	}

	return false;
}
