// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMeshPaintGeometryAdapter.h"
#include "Containers/Array.h"
#include "IMeshPaintGeometryAdapterFactory.h"
#include "Math/Matrix.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FReferenceCollector;
class IMeshPaintGeometryAdapter;
class UClothingAssetBase;
class UClothingAssetCommon;
class UMeshComponent;
class USkeletalMesh;
class USkeletalMeshComponent;
class UTexture;
struct FColor;
struct FGuid;
struct FPointWeightMap;

/** Adapter used to paint simulation specific properties to cloth assets inside of a Skeletal mesh */
class FClothMeshPaintAdapter : public FBaseMeshPaintGeometryAdapter
{
protected:
	struct FClothAssetInfo
	{
		/** Begin/End for this asset's verts in the provided buffer. */
		int32 VertexStart;
		int32 VertexEnd;

		/** Begin/End for this asset's indices in the provided buffer. */
		int32 IndexStart;
		int32 IndexEnd;

		/** Map of index to neigbor indices */
		TArray<TArray<int32>> NeighborMap;

		/** The actual clothing asset relating to this data */
		UClothingAssetCommon* Asset;
	};
public:
	static void InitializeAdapterGlobals() {}
	static void AddReferencedObjectsGlobals(FReferenceCollector& Collector) {}
	static void CleanupGlobals() {}

	virtual bool Construct(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex) override;
	virtual bool Initialize() override;
	virtual void OnAdded() override {}
	virtual void OnRemoved() override {}
	virtual bool IsValid() const override { return true; }
	
	virtual bool SupportsTexturePaint() const override { return false; }
	virtual bool SupportsVertexPaint() const override { return true; }

	virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;
	virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {}
	
	virtual void PreEdit() override;
	virtual void PostEdit() override;
	
	virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	virtual FMatrix GetComponentToWorldMatrix() const override;

	virtual TArray<FVector> SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const;

	/** Sets the represented clothing asset to the UClothingAssetBase retrieved from the AssetGUID */
	virtual void SetSelectedClothingAsset(const FGuid& InAssetGuid, int32 InAssetLod, int32 InMaskIndex);

	/** Gets a list of the neighbors of the specified vertex */
	const TArray<int32>* GetVertexNeighbors(int32 InVertexIndex) const;

	/** Get the current mask we're editing */
	FPointWeightMap* GetCurrentMask() const;

protected:

	/** Initialize adapter data ready for painting */
	virtual bool InitializeVertexData();

	/** Whether or not our current asset/lod/mask selection has a valid paintable surface */
	bool HasValidSelection() const;

protected:
	/** (Debug) Skeletal Mesh Component this adapter represents */
	USkeletalMeshComponent* SkeletalMeshComponent;
	/** Skeletal mesh asset this adapter represents */
	USkeletalMesh* ReferencedSkeletalMesh;

	/** LOD index to paint to (cloth LOD data) */
	int32 PaintingClothLODIndex;

	/** Mask inside the current LOD to paint */
	int32 PaintingClothMaskIndex;

	/** Currently selected clothing asset object to paint to */
	UClothingAssetBase* SelectedAsset;

	/** List of clothing assets objects contained by ReferenceSkeletalMesh */
	TArray<UClothingAssetBase*> ClothingAssets;
	/** List of clothing asset info structs contained by ReferenceSkeletalMesh */
	TArray<FClothAssetInfo> AssetInfoMap;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintSpriteAdapterFactory

class FClothMeshPaintAdapterFactory : public IMeshPaintGeometryAdapterFactory
{
public:
	virtual TSharedPtr<IMeshPaintGeometryAdapter> Construct(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FClothMeshPaintAdapter::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FClothMeshPaintAdapter::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FClothMeshPaintAdapter::CleanupGlobals(); }
};
