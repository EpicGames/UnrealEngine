// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPaintSettings.h"
#include "Delegates/Delegate.h"

#include "ClothPaintSettings.generated.h"

class UClothingAssetCommon;

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnClothingAssetSelectionChangedMulticaster, UClothingAssetCommon*, int32, int32);
typedef FOnClothingAssetSelectionChangedMulticaster::FDelegate FOnClothingAssetSelectionChanged;

UENUM()
enum class EPaintableClothProperty
{
	/** Max distances cloth property */
	MaxDistances,
	/** Backstop cloth property */
	BackstopDistances,
	/** Backstop radius property */
	BackstopRadius
};

UCLASS(config = EditorPerProjectUserSettings)
class UClothPainterSettings : public UMeshPaintSettings
{
	GENERATED_BODY()

public:

	UClothPainterSettings()
		: UMeshPaintSettings()
		, ViewMin(0.0f)
		, ViewMax(100.0f)
		, bAutoViewRange(false)
		, AutoCalculatedViewMin(0.0f)
		, AutoCalculatedViewMax(0.0f)
		, bFlipNormal(false)
		, bCullBackface(false)
		, Opacity(1.f)
	{}

	float GetViewMin()
	{
		float TempViewMin = bAutoViewRange ? AutoCalculatedViewMin : ViewMin;

		// Zero is reserved, but conceptually we should allow it as that's an
		// implementation detail the user is unlikely to care about
		return FMath::Clamp(TempViewMin, SMALL_NUMBER, MAX_flt);
	}

	float GetViewMax()
	{
		return bAutoViewRange ? AutoCalculatedViewMax : ViewMax;
	}

	// Delegates to communicate with objects concerned with the settings changing
	FOnClothingAssetSelectionChangedMulticaster OnAssetSelectionChanged;

protected:
	/** When painting float/1D values, this is considered the zero or black point */
	UPROPERTY(EditAnywhere, Config, Category = View, meta = (UIMin = 0, UIMax = 100000, ClampMin = 0, ClampMax = 100000, EditCondition = "!bAutoViewRange"))
	float ViewMin;

	/** When painting float/1D values, this is considered the one or white point */
	UPROPERTY(EditAnywhere, Config, Category = View, meta = (UIMin = 0, UIMax = 100000, ClampMin = 0, ClampMax = 100000, EditCondition = "!bAutoViewRange"))
	float ViewMax;

public:

	/** When set, the view min and max values will be calculated from the values present in the currently editable mask */
	UPROPERTY(EditAnywhere, Category = View)
	bool bAutoViewRange;

	/** Storage for auto calculated view min value */
	UPROPERTY()
	float AutoCalculatedViewMin;

	/** Storage for auto calculated view max value */
	UPROPERTY()
	float AutoCalculatedViewMax;

	/** Array of Clothing assets */
	UPROPERTY()
	TArray<TObjectPtr<UClothingAssetCommon>> ClothingAssets;

	/** Whether to flip normals on the mesh preview */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = View)
	bool bFlipNormal;

	/** Whether to cull backfacing triangles when rendering the mesh preview */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = View)
	bool bCullBackface;

	/** Opacity of the mesh preview */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = View, meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1))
	float Opacity;
};
