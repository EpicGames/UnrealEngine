// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPaintSettingsCustomization.h"

#include "ClothPaintSettings.h"
#include "ClothPaintToolBase.h"
#include "ClothPainter.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "MeshPaintSettings.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "ClothPaintSettingsCustomization"

FClothPaintSettingsCustomization::FClothPaintSettingsCustomization(FClothPainter* InPainter) 
	: Painter(InPainter)
{

}

FClothPaintSettingsCustomization::~FClothPaintSettingsCustomization()
{
	
}

TSharedRef<IDetailCustomization> FClothPaintSettingsCustomization::MakeInstance(FClothPainter* InPainter)
{
	return MakeShareable(new FClothPaintSettingsCustomization(InPainter));
}

void FClothPaintSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);
	
	UClothPainterSettings* PainterSettings = nullptr;
	for(TWeakObjectPtr<UObject>& WeakObj : CustomizedObjects)
	{
		if(UObject* CustomizedObject = WeakObj.Get())
		{
			if(UClothPainterSettings* Settings = Cast<UClothPainterSettings>(CustomizedObject))
			{
				PainterSettings = Settings;
			}
		}
	}

	const FName ClothCategory = "ClothPainting";
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ClothCategory);

	// Add tool selection from the tools array on the painter
	FText ToolText = LOCTEXT("ToolSelectionRow", "Tool");
	FDetailWidgetRow& ToolSelectionRow = CategoryBuilder.AddCustomRow(ToolText);

	ToolSelectionRow.NameContent()
		[
			SNew(STextBlock)
				.Text(ToolText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	ToolSelectionRow.ValueContent()
		[
			SNew(SComboBox<TSharedPtr<FClothPaintToolBase>>)
				.OptionsSource(&Painter->Tools)
				.OnGenerateWidget(this, &FClothPaintSettingsCustomization::OnGenerateToolComboRow)
				.OnSelectionChanged(this, &FClothPaintSettingsCustomization::OnHandleToolSelection, &DetailBuilder)
				[
					SNew(STextBlock)
						.Text(this, &FClothPaintSettingsCustomization::GetToolComboText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];

	TSharedRef<IPropertyHandle> AutoRangeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UClothPainterSettings, bAutoViewRange), UClothPainterSettings::StaticClass());

	if(AutoRangeProperty->IsValidHandle())
	{
		AutoRangeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FClothPaintSettingsCustomization::OnAutoRangeFlagChanged));
	}
}

TSharedRef<SWidget> FClothPaintSettingsCustomization::OnGenerateToolComboRow(TSharedPtr<FClothPaintToolBase> InItem)
{
	FText RowText = LOCTEXT("ToolComboRow_Error", "Invalid");

	if(InItem.IsValid())
	{
		RowText = InItem->GetDisplayName();
	}

	return SNew(STextBlock).Text(RowText);
}

void FClothPaintSettingsCustomization::OnHandleToolSelection(TSharedPtr<FClothPaintToolBase> InItem, ESelectInfo::Type InSelectInfo, IDetailLayoutBuilder* DetailBuider)
{
	if(InItem.IsValid() && Painter && Painter->SelectedTool != InItem)
	{
		// Update selection
		Painter->SetTool(InItem);
		Painter->Refresh();
	}
}

FText FClothPaintSettingsCustomization::GetToolComboText() const
{
	if(Painter && Painter->SelectedTool.IsValid())
	{
		return Painter->SelectedTool->GetDisplayName();
	}

	return LOCTEXT("ToolComboRow_Error", "Invalid");
}

void FClothPaintSettingsCustomization::OnAutoRangeFlagChanged()
{
	Painter->RecalculateAutoViewRange();
}

#undef LOCTEXT_NAMESPACE

TSharedRef<IDetailCustomization> FClothPaintBrushSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FClothPaintBrushSettingsCustomization());
}

void FClothPaintBrushSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> ColorViewProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPaintBrushSettings, ColorViewMode));

	ColorViewProp->MarkHiddenByCustomization();
}
