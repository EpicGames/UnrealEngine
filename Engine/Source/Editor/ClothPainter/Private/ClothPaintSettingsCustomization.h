// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FClothPaintToolBase;
class FClothPainter;
class IDetailLayoutBuilder;
class SWidget;

class FClothPaintSettingsCustomization : public IDetailCustomization
{
public:

	FClothPaintSettingsCustomization() = delete;
	FClothPaintSettingsCustomization(FClothPainter* InPainter);

	virtual ~FClothPaintSettingsCustomization();

	static TSharedRef<IDetailCustomization> MakeInstance(FClothPainter* InPainter);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	/** Handlers for SCombobox, generation of row widgets, combo text and handling selection change */
	TSharedRef<SWidget> OnGenerateToolComboRow(TSharedPtr<FClothPaintToolBase> InItem);
	void OnHandleToolSelection(TSharedPtr<FClothPaintToolBase> InItem, ESelectInfo::Type InSelectInfo, IDetailLayoutBuilder* DetailBuider);
	FText GetToolComboText() const;

	/** Callback to calculate auto view range values */
	void OnAutoRangeFlagChanged();

	/** The painter containing the paint settings we are customizing */
	FClothPainter* Painter;
};


class FClothPaintBrushSettingsCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};
