// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPaintHelpers.h"
#include "ClothPainter.h"

class IDetailsView;

/** 
 * Base object for tools used to paint clothing
 * Derive from this and register a new tool in FClothPainter::Init to add to the available tools
 */
class FClothPaintToolBase
{
public:

	FClothPaintToolBase() = delete;
	FClothPaintToolBase(const FClothPaintToolBase& Other) = delete;

	explicit FClothPaintToolBase(TWeakPtr<FClothPainter> InPainter)
		: Painter(InPainter)
	{}

	virtual ~FClothPaintToolBase() = default;

	/** Used by the painter to request a delegate to call to apply this tool to the current mesh */
	virtual FPerVertexPaintAction GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings) { return FPerVertexPaintAction(); }

	/** Called when the user presses a key when this tool is selected */
	virtual bool InputKey(IMeshPaintGeometryAdapter* Adapter, FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) { return false; }

	/** Whether or not the brush interactor should be drawn */
	virtual bool ShouldRenderInteractors() const { return true; }

	/** Display name for UI purposes */
	virtual FText GetDisplayName() const = 0;

	/** Whether this action should be executed once for each vertex in the brush or just once per operation */
	virtual bool IsPerVertex() const { return true; }

	/** Optionally render extra data to the viewport */
	virtual void Render(USkeletalMeshComponent* InComponent, IMeshPaintGeometryAdapter* InAdapter, const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) {}

	/** Called as tool is selected, can be used to initialize and bind actions */
	virtual void Activate(TWeakPtr<FUICommandList> InCommands) {};

	/** Called as tool is deselected, can be used to shutdown and unbind actions */
	virtual void Deactivate(TWeakPtr<FUICommandList> InCommands) {};

	/** Called as a change in the target geometry changes, to clear anything the tool may have cached */
	virtual void OnMeshChanged() {};

	/** If true, the tool must provide the range of values used in GetValueRange, used to modify auto calculated view ranges */
	virtual bool HasValueRange() { return false; }

	/** If HasValueRange returns true, this should specify the range of values the tool is currently covering */
	virtual void GetValueRange(float& OutRangeMin, float& OutRangeMax) {}

	/** 
	 * Optionally return a UObject that will be displayed in the details panel when the tool is selected.
	 * This is intended for settings unique to the tool, common settings (brush size etc) are available from
	 * the brush settings in the Painter.
	 */
	virtual UObject* GetSettingsObject() { return nullptr; }

	/** Optionally register any applicable customizations for the settings object */
	virtual void RegisterSettingsObjectCustomizations(IDetailsView* InDetailsView) {}

protected:

	/** The painter instance that owns this tool */
	TWeakPtr<FClothPainter> Painter;
};
