// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPaintToolCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "ClothPaintToolCommands"

void ClothPaintToolCommands::RegisterClothPaintToolCommands()
{
	FClothPaintToolCommands_Gradient::Register();
}

void FClothPaintToolCommands_Gradient::RegisterCommands()
{
	UI_COMMAND(ApplyGradient, "Apply gradient", "Apply the gradient when the clothing paint gradient tool is active.", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
}

const FClothPaintToolCommands_Gradient& FClothPaintToolCommands_Gradient::Get()
{
	return TCommands<FClothPaintToolCommands_Gradient>::Get();
}

#undef LOCTEXT_NAMESPACE
