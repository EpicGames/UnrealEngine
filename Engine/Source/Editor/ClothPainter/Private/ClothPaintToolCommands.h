// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

namespace ClothPaintToolCommands
{
	void RegisterClothPaintToolCommands();
}

class FClothPaintToolCommands_Gradient : public TCommands<FClothPaintToolCommands_Gradient>
{
public:

	FClothPaintToolCommands_Gradient()
		: TCommands<FClothPaintToolCommands_Gradient>(TEXT("ClothPainter"), NSLOCTEXT("Contexts", "ClothFillTool", "Cloth Painter - Fill Tool"), NAME_None, FAppStyle::GetAppStyleSetName())
	{

	}

	virtual void RegisterCommands() override;
	static const FClothPaintToolCommands_Gradient& Get();

	/** Applies the gradient when using the gradient cloth paint tool */
	TSharedPtr<FUICommandInfo> ApplyGradient;
};