// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPaintTools.h"

#include "ClothMeshAdapter.h"
#include "ClothPaintSettings.h"
#include "ClothPaintToolCommands.h"
#include "ClothPainter.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/EngineTypes.h"
#include "Engine/HitResult.h"
#include "Engine/NetSerialization.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "IMeshPaintGeometryAdapter.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintSettings.h"
#include "MeshPaintTypes.h"
#include "Misc/AssertionMacros.h"
#include "PropertyEditorDelegates.h"
#include "PrimitiveDrawingUtils.h"
#include "ScopedTransaction.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothPaintTools)

class FEditorViewportClient;
class FSceneView;
class FViewport;
class IPropertyHandle;

#define LOCTEXT_NAMESPACE "ClothTools"

FClothPaintTool_Brush::~FClothPaintTool_Brush()
{
	if(Settings)
	{
		Settings->RemoveFromRoot();
	}
}

FText FClothPaintTool_Brush::GetDisplayName() const
{
	return LOCTEXT("ToolName_Brush", "Brush");
}

FPerVertexPaintAction FClothPaintTool_Brush::GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings)
{
	return FPerVertexPaintAction::CreateRaw(this, &FClothPaintTool_Brush::PaintAction, InPaintParams.InverseBrushToWorldMatrix);
}

UObject* FClothPaintTool_Brush::GetSettingsObject()
{
	if(!Settings)
	{
		Settings = DuplicateObject<UClothPaintTool_BrushSettings>(GetMutableDefault<UClothPaintTool_BrushSettings>(), GetTransientPackage());
		Settings->AddToRoot();
	}

	return Settings;
}

bool FClothPaintTool_Brush::HasValueRange()
{
	return true;
}

void FClothPaintTool_Brush::GetValueRange(float& OutRangeMin, float& OutRangeMax)
{
	OutRangeMin = Settings->PaintValue;
	OutRangeMax = OutRangeMin;
}

void FClothPaintTool_Brush::PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix)
{
	FClothMeshPaintAdapter* ClothAdapter = (FClothMeshPaintAdapter*)InArgs.Adapter;
	if(ClothAdapter)
	{
		TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();
		if(SharedPainter.IsValid())
		{
			UPaintBrushSettings* BrushSettings = SharedPainter->GetBrushSettings();
			UClothPainterSettings* PaintSettings = Cast<UClothPainterSettings>(SharedPainter->GetPainterSettings());

			FVector Position;
			ClothAdapter->GetVertexPosition(VertexIndex, Position);
			Position = ClothAdapter->GetComponentToWorldMatrix().TransformPosition(Position);

			float Value = SharedPainter->GetPropertyValue(VertexIndex);
			const float BrushRadius = BrushSettings->GetBrushRadius();
			MeshPaintHelpers::ApplyBrushToVertex(Position, InverseBrushMatrix, BrushRadius, BrushSettings->BrushFalloffAmount, BrushSettings->BrushStrength, Settings->PaintValue, Value);
			SharedPainter->SetPropertyValue(VertexIndex, Value);
		}
	}
}

FClothPaintTool_Gradient::~FClothPaintTool_Gradient()
{
	if(Settings)
	{
		Settings->RemoveFromRoot();
	}
}

FText FClothPaintTool_Gradient::GetDisplayName() const
{
	return LOCTEXT("ToolName_Gradient", "Gradient");
}

bool FClothPaintTool_Gradient::InputKey(IMeshPaintGeometryAdapter* Adapter, FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
	bool bHandled = false;

	if(InKey == EKeys::LeftControl || InKey == EKeys::RightControl)
	{
		if(InEvent == IE_Pressed)
		{
			bSelectingBeginPoints = false;
			bHandled = true;
		}
		else if(InEvent == IE_Released)
		{
			bSelectingBeginPoints = true;
			bHandled = true;
		}
	}

	return bHandled;
}

FPerVertexPaintAction FClothPaintTool_Gradient::GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings)
{
	return FPerVertexPaintAction::CreateRaw(this, &FClothPaintTool_Gradient::PaintAction, InPaintParams.InverseBrushToWorldMatrix);
}

bool FClothPaintTool_Gradient::IsPerVertex() const
{
	return false;
}

void FClothPaintTool_Gradient::Render(USkeletalMeshComponent* InComponent, IMeshPaintGeometryAdapter* InAdapter, const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();

	if(!SharedPainter.IsValid())
	{
		return;
	}

	const float VertexPointSize = GetDefault<UMeshPaintSettings>()->VertexPreviewSize;
	const UClothPainterSettings* PaintSettings = CastChecked<UClothPainterSettings>(SharedPainter->GetPainterSettings());
	const UPaintBrushSettings* BrushSettings = SharedPainter->GetBrushSettings();

	TArray<MeshPaintHelpers::FPaintRay> PaintRays;
	MeshPaintHelpers::RetrieveViewportPaintRays(View, Viewport, PDI, PaintRays);
	
	const FMatrix ComponentToWorldMatrix = InComponent->GetComponentTransform().ToMatrixWithScale();
	
	for (const int32& Index : GradientStartIndices)
	{
		FVector Vertex;
		InAdapter->GetVertexPosition(Index, Vertex);

		const FVector WorldPositionVertex = ComponentToWorldMatrix.TransformPosition(Vertex);
		PDI->DrawPoint(WorldPositionVertex, FLinearColor::Green, VertexPointSize, SDPG_World);
	}
	
	for (const int32& Index : GradientEndIndices)
	{
		FVector Vertex;
		InAdapter->GetVertexPosition(Index, Vertex);

		const FVector WorldPositionVertex = ComponentToWorldMatrix.TransformPosition(Vertex);
		PDI->DrawPoint(WorldPositionVertex, FLinearColor::Red, VertexPointSize, SDPG_World);
	}
	
	
	for (const MeshPaintHelpers::FPaintRay& PaintRay : PaintRays)
	{
		const FHitResult& HitResult = SharedPainter->GetHitResult(PaintRay.RayStart, PaintRay.RayDirection);
		if (HitResult.Component == InComponent)
		{
			const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(PaintRay.CameraLocation));
			const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));
			const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(Settings->bUseRegularBrush ? BrushSettings->GetBrushRadius() : 2.0f, 0.0f, 0.0f)).Size();
			const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;
	
			// Draw hovered vertex
			TArray<FVector> InRangeVertices = InAdapter->SphereIntersectVertices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles);
			InRangeVertices.Sort([=](const FVector& PointOne, const FVector& PointTwo)
			{
				return (PointOne - ComponentSpaceBrushPosition).SizeSquared() < (PointTwo - ComponentSpaceBrushPosition).SizeSquared();
			});
	
			
			if (InRangeVertices.Num())
			{
				if (!Settings->bUseRegularBrush)
				{
					InRangeVertices.SetNum(1);
				}
	
				for (const FVector& Vertex : InRangeVertices)
				{
					const FVector WorldPositionVertex = ComponentToWorldMatrix.TransformPosition(Vertex);
					PDI->DrawPoint(WorldPositionVertex, bSelectingBeginPoints ? FLinearColor::Green : FLinearColor::Red, VertexPointSize, SDPG_Foreground);
				}
			}
		}
	}
}

bool FClothPaintTool_Gradient::ShouldRenderInteractors() const
{
	return Settings->bUseRegularBrush;
}

UObject* FClothPaintTool_Gradient::GetSettingsObject()
{
	if(!Settings)
	{
		Settings = DuplicateObject<UClothPaintTool_GradientSettings>(GetMutableDefault<UClothPaintTool_GradientSettings>(), GetTransientPackage());
		Settings->AddToRoot();
	}

	return Settings;
}

void FClothPaintTool_Gradient::Activate(TWeakPtr<FUICommandList> InCommands)
{
	GradientStartIndices.Empty();
	GradientEndIndices.Empty();

	TSharedPtr<FUICommandList> SharedCommands = InCommands.Pin();
	if(SharedCommands.IsValid())
	{
		const FClothPaintToolCommands_Gradient& Commands = FClothPaintToolCommands_Gradient::Get();

		SharedCommands->MapAction(Commands.ApplyGradient, 
			FExecuteAction::CreateRaw(this, &FClothPaintTool_Gradient::ApplyGradient),
			FCanExecuteAction::CreateRaw(this, &FClothPaintTool_Gradient::CanApplyGradient)
		);
	}
}

void FClothPaintTool_Gradient::Deactivate(TWeakPtr<FUICommandList> InCommands)
{
	TSharedPtr<FUICommandList> SharedCommands = InCommands.Pin();
	if(SharedCommands.IsValid())
	{
		const FClothPaintToolCommands_Gradient& Commands = FClothPaintToolCommands_Gradient::Get();

		SharedCommands->UnmapAction(Commands.ApplyGradient);
	}
}

void FClothPaintTool_Gradient::OnMeshChanged()
{
	GradientStartIndices.Empty();
	GradientEndIndices.Empty();
}

bool FClothPaintTool_Gradient::HasValueRange()
{
	return true;
}

void FClothPaintTool_Gradient::GetValueRange(float& OutRangeMin, float& OutRangeMax)
{
	OutRangeMin = FMath::Min(Settings->GradientStartValue, Settings->GradientEndValue);
	OutRangeMax = FMath::Max(Settings->GradientStartValue, Settings->GradientEndValue);
}

void FClothPaintTool_Gradient::PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix)
{
	TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();
	if(!SharedPainter.IsValid())
	{
		return;
	}

	const FHitResult HitResult = InArgs.HitResult;
	UClothPainterSettings* PaintSettings = CastChecked<UClothPainterSettings>(SharedPainter->GetPainterSettings());
	const UPaintBrushSettings* BrushSettings = InArgs.BrushSettings;
	FClothMeshPaintAdapter* Adapter = (FClothMeshPaintAdapter*)InArgs.Adapter;

	const FMatrix ComponentToWorldMatrix = HitResult.Component->GetComponentTransform().ToMatrixWithScale();
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(Settings->bUseRegularBrush ? BrushSettings->GetBrushRadius() : 2.0f, 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;
	
	// Override these flags becuase we're not actually "painting" so the state would be incorrect
	SharedPainter->SetIsPainting(true);
	
	TArray<FVector> InRangeVertices = Adapter->SphereIntersectVertices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles);

	TSet<int32> InRangeIndicesSet;
	Adapter->GetInfluencedVertexIndices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles, InRangeIndicesSet);
	
	// Must have an array for sorting/counting
	TArray<int32> InRangeIndices = InRangeIndicesSet.Array();

	if (InRangeIndices.Num())
	{
		// Sort based on distance from brush centre
		InRangeIndices.Sort([=](const int32& Idx0, const int32& Idx1)
		{
			FVector PointOne;
			FVector PointTwo;

			Adapter->GetVertexPosition(Idx0, PointOne);
			Adapter->GetVertexPosition(Idx1, PointTwo);

			return (PointOne - ComponentSpaceBrushPosition).SizeSquared() < (PointTwo - ComponentSpaceBrushPosition).SizeSquared();
		});
	
		// Selection mode
		if (!Settings->bUseRegularBrush)
		{
			InRangeIndices.SetNum(1);
		}
	
		TArray<int32>& CurrentList = bSelectingBeginPoints ? GradientStartIndices : GradientEndIndices;
		TArray<int32>& OtherList = bSelectingBeginPoints ? GradientEndIndices : GradientStartIndices;

		// Add selected verts to current list
		for (const int32& Index : InRangeIndices)
		{
			if (InArgs.Action == EMeshPaintAction::Erase && CurrentList.Contains(Index))
			{
				CurrentList.Remove(Index);
			}
			else if (InArgs.Action == EMeshPaintAction::Paint)
			{
				CurrentList.AddUnique(Index);
				OtherList.Remove(Index);
			}
		}
	}
}

void FClothPaintTool_Gradient::ApplyGradient()
{
	TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();
	if(!SharedPainter.IsValid())
	{
		return;
	}

	TSharedPtr<IMeshPaintGeometryAdapter> SharedAdapter = SharedPainter->GetAdapter();

	if(!SharedAdapter.IsValid())
	{
		return;
	}

	FClothMeshPaintAdapter* Adapter = (FClothMeshPaintAdapter*)SharedAdapter.Get();

	{
		FScopedTransaction Transaction(LOCTEXT("ApplyGradientTransaction", "Apply gradient"));

		Adapter->PreEdit();

		const TArray<FVector> Verts = Adapter->GetMeshVertices();

		const int32 NumVerts = Verts.Num();
		for(int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
		{
			const FVector& Vert = Verts[VertIndex];

			// Get distances
			// TODO: Look into surface distance instead of 3D distance? May be necessary for some complex shapes
			float DistanceToStartSq = MAX_flt;
			for(const int32& StartIndex : GradientStartIndices)
			{
				FVector StartPoint;
				Adapter->GetVertexPosition(StartIndex, StartPoint);
				const float DistanceSq = (StartPoint - Vert).SizeSquared();
				if(DistanceSq < DistanceToStartSq)
				{
					DistanceToStartSq = DistanceSq;
				}
			}

			float DistanceToEndSq = MAX_flt;
			for(const int32& EndIndex : GradientEndIndices)
			{
				FVector EndPoint;
				Adapter->GetVertexPosition(EndIndex, EndPoint);
				const float DistanceSq = (EndPoint - Vert).SizeSquared();
				if(DistanceSq < DistanceToEndSq)
				{
					DistanceToEndSq = DistanceSq;
				}
			}

			// Apply to the mesh
			UClothPainterSettings* PaintSettings = CastChecked<UClothPainterSettings>(SharedPainter->GetPainterSettings());
			const float Value = FMath::LerpStable(Settings->GradientStartValue, Settings->GradientEndValue, DistanceToStartSq / (DistanceToStartSq + DistanceToEndSq));
			SharedPainter->SetPropertyValue(VertIndex, Value);
		}
		
		// Finished edit
		Adapter->PostEdit();
	}

	// Empty the point lists as the operation is complete
	GradientStartIndices.Reset();
	GradientEndIndices.Reset();

	// Move back to selecting begin points
	bSelectingBeginPoints = true;
}

bool FClothPaintTool_Gradient::CanApplyGradient()
{
	return GradientEndIndices.Num() > 0 && GradientStartIndices.Num() > 0;
}

class FSmoothToolCustomization : public IDetailCustomization
{
public:
	FSmoothToolCustomization() = delete;

	FSmoothToolCustomization(TSharedPtr<FClothPainter> InPainter)
		: Painter(InPainter)
	{}

	static TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<FClothPainter> InPainter)
	{
		return MakeShareable(new FSmoothToolCustomization(InPainter));
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		const FName ToolCategoryName = "ToolSettings";
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ToolCategoryName);

		TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
		CategoryBuilder.GetDefaultProperties(DefaultProperties);

		for(TSharedRef<IPropertyHandle>& Handle : DefaultProperties)
		{
			CategoryBuilder.AddProperty(Handle);
		}

		FDetailWidgetRow& MeshSmoothRow = CategoryBuilder.AddCustomRow(LOCTEXT("MeshSmoothRowName", "MeshSmooth"));

		MeshSmoothRow.ValueContent()
			[
				SNew(SButton)
				.Text(LOCTEXT("MeshSmoothButtonText", "Smooth Mesh"))
				.ToolTipText(LOCTEXT("MeshSmoothButtonToolTip", "Applies the smooth operation to the whole mesh at once."))
				.OnClicked(this, &FSmoothToolCustomization::OnMeshSmoothClicked)
			];
	}

private:

	FReply OnMeshSmoothClicked()
	{
		if(Painter.IsValid())
		{
			const int32 NumVerts = Painter->GetAdapter()->GetMeshVertices().Num();
			TSet<int32> IndexSet;
			IndexSet.Reserve(NumVerts);

			for(int32 Index = 0; Index < NumVerts; ++Index)
			{
				IndexSet.Add(Index);
			}

			TSharedPtr<FClothPaintTool_Smooth> SmoothTool = StaticCastSharedPtr<FClothPaintTool_Smooth>(Painter->GetSelectedTool());
			if(SmoothTool.IsValid())
			{
				SmoothTool->SmoothVertices(IndexSet, Painter);
			}
		}

		return FReply::Handled();
	}

	TSharedPtr<FClothPainter> Painter;
};

FClothPaintTool_Smooth::~FClothPaintTool_Smooth()
{
	if(Settings)
	{
		Settings->RemoveFromRoot();
	}
}

FPerVertexPaintAction FClothPaintTool_Smooth::GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings)
{
	return FPerVertexPaintAction::CreateRaw(this, &FClothPaintTool_Smooth::PaintAction, InPaintParams.InverseBrushToWorldMatrix);
}

FText FClothPaintTool_Smooth::GetDisplayName() const
{
	return LOCTEXT("ToolName_Smooth", "Smooth");
}

UObject* FClothPaintTool_Smooth::GetSettingsObject()
{
	if(!Settings)
	{
		Settings = DuplicateObject<UClothPaintTool_SmoothSettings>(GetMutableDefault<UClothPaintTool_SmoothSettings>(), GetTransientPackage());
		Settings->AddToRoot();
	}

	return Settings;
}

bool FClothPaintTool_Smooth::IsPerVertex() const
{
	return false;
}

void FClothPaintTool_Smooth::RegisterSettingsObjectCustomizations(IDetailsView* InDetailsView)
{
	InDetailsView->RegisterInstancedCustomPropertyLayout(UClothPaintTool_SmoothSettings::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FSmoothToolCustomization::MakeInstance, Painter.Pin()));
}

void FClothPaintTool_Smooth::PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix)
{
	TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();
	if(!SharedPainter.IsValid())
	{
		return;
	}

	const FHitResult HitResult = InArgs.HitResult;
	UClothPainterSettings* PaintSettings = CastChecked<UClothPainterSettings>(SharedPainter->GetPainterSettings());
	const UPaintBrushSettings* BrushSettings = InArgs.BrushSettings;
	FClothMeshPaintAdapter* Adapter = (FClothMeshPaintAdapter*)InArgs.Adapter;

	const FMatrix ComponentToWorldMatrix = HitResult.Component->GetComponentTransform().ToMatrixWithScale();
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushSettings->GetBrushRadius(), 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

	// Override these flags becuase we're not actually "painting" so the state would be incorrect
	SharedPainter->SetIsPainting(true);

	TSet<int32> InfluencedVertices;
	Adapter->GetInfluencedVertexIndices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles, InfluencedVertices);
	SmoothVertices(InfluencedVertices, SharedPainter);
}

void FClothPaintTool_Smooth::SmoothVertices(const TSet<int32> &InfluencedVertices, TSharedPtr<FClothPainter> SharedPainter)
{
	check(SharedPainter.IsValid());

	TSharedPtr<IMeshPaintGeometryAdapter> AdapterInterface = SharedPainter->GetAdapter();
	FClothMeshPaintAdapter* Adapter = (FClothMeshPaintAdapter*)AdapterInterface.Get();

	const int32 NumVerts = InfluencedVertices.Num();
	if(NumVerts > 0)
	{
		TArray<float> NewValues;
		NewValues.AddZeroed(NumVerts);

		int32 InfluencedIndex = 0;
		for(const int32 Index : InfluencedVertices)
		{
			const TArray<int32>* Neighbors = Adapter->GetVertexNeighbors(Index);
			if(Neighbors && Neighbors->Num() > 0)
			{
				float Accumulator = 0.0f;

				for(int32 NeighborIndex : (*Neighbors))
				{
					Accumulator += SharedPainter->GetPropertyValue(NeighborIndex);
				}

				NewValues[InfluencedIndex] = Accumulator / Neighbors->Num();
			}
			else
			{
				NewValues[InfluencedIndex] = SharedPainter->GetPropertyValue(Index);
			}

			++InfluencedIndex;
		}

		int32 ValueIndex = 0;
		for(int32 Index : InfluencedVertices)
		{
			float Current = SharedPainter->GetPropertyValue(Index);
			float Difference = NewValues[ValueIndex] - Current;

			SharedPainter->SetPropertyValue(Index, Current + Difference * Settings->Strength);
			++ValueIndex;
		}
	}
}

FClothPaintTool_Fill::~FClothPaintTool_Fill()
{
	if(Settings)
	{
		Settings->RemoveFromRoot();
	}
}

FPerVertexPaintAction FClothPaintTool_Fill::GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings)
{
	return FPerVertexPaintAction::CreateRaw(this, &FClothPaintTool_Fill::PaintAction, InPaintParams.InverseBrushToWorldMatrix);
}

FText FClothPaintTool_Fill::GetDisplayName() const
{
	return LOCTEXT("ToolName_Fill", "Fill");
}

UObject* FClothPaintTool_Fill::GetSettingsObject()
{
	if(!Settings)
	{
		Settings = DuplicateObject<UClothPaintTool_FillSettings>(GetMutableDefault<UClothPaintTool_FillSettings>(), GetTransientPackage());
		Settings->AddToRoot();
	}

	return Settings;
}

bool FClothPaintTool_Fill::IsPerVertex() const
{
	return false;
}

bool FClothPaintTool_Fill::ShouldRenderInteractors() const
{
	return false;
}

void FClothPaintTool_Fill::Render(USkeletalMeshComponent* InComponent, IMeshPaintGeometryAdapter* InAdapter, const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();

	if(!SharedPainter.IsValid())
	{
		return;
	}

	const float VertexPointSize = GetDefault<UMeshPaintSettings>()->VertexPreviewSize;
	const UClothPainterSettings* PaintSettings = CastChecked<UClothPainterSettings>(SharedPainter->GetPainterSettings());
	const UPaintBrushSettings* BrushSettings = SharedPainter->GetBrushSettings();

	TArray<MeshPaintHelpers::FPaintRay> PaintRays;
	MeshPaintHelpers::RetrieveViewportPaintRays(View, Viewport, PDI, PaintRays);

	const FMatrix ComponentToWorldMatrix = InComponent->GetComponentTransform().ToMatrixWithScale();

	for(const MeshPaintHelpers::FPaintRay& PaintRay : PaintRays)
	{
		const FHitResult& HitResult = SharedPainter->GetHitResult(PaintRay.RayStart, PaintRay.RayDirection);
		if(HitResult.Component == InComponent)
		{
			const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(PaintRay.CameraLocation));
			const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));
			const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(QueryRadius, 0.0f, 0.0f)).Size();
			const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

			// Draw hovered vertex
			TArray<FVector> InRangeVertices = InAdapter->SphereIntersectVertices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles);
			InRangeVertices.Sort([=](const FVector& PointOne, const FVector& PointTwo)
			{
				return (PointOne - ComponentSpaceBrushPosition).SizeSquared() < (PointTwo - ComponentSpaceBrushPosition).SizeSquared();
			});


			if(InRangeVertices.Num())
			{
				InRangeVertices.SetNum(1);
				// reduce this
				for(const FVector& Vertex : InRangeVertices)
				{
					const FVector WorldPositionVertex = ComponentToWorldMatrix.TransformPosition(Vertex);
					PDI->DrawPoint(WorldPositionVertex, FLinearColor::Green, VertexPointSize, SDPG_Foreground);
				}
			}
		}
	}
}

bool FClothPaintTool_Fill::HasValueRange()
{
	return true;
}

void FClothPaintTool_Fill::GetValueRange(float& OutRangeMin, float& OutRangeMax)
{
	OutRangeMin = Settings->FillValue;
	OutRangeMax = OutRangeMin;
}

void FClothPaintTool_Fill::PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix)
{
	TSharedPtr<FClothPainter> SharedPainter = Painter.Pin();
	if(!SharedPainter.IsValid())
	{
		return;
	}

	const FHitResult HitResult = InArgs.HitResult;
	UClothPainterSettings* PaintSettings = CastChecked<UClothPainterSettings>(SharedPainter->GetPainterSettings());
	const UPaintBrushSettings* BrushSettings = InArgs.BrushSettings;
	FClothMeshPaintAdapter* Adapter = (FClothMeshPaintAdapter*)InArgs.Adapter;

	const FMatrix ComponentToWorldMatrix = HitResult.Component->GetComponentTransform().ToMatrixWithScale();
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(QueryRadius, 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

	// Override these flags becuase we're not actually "painting" so the state would be incorrect
	SharedPainter->SetIsPainting(true);

	TArray<TPair<int32, FVector>> Verts;
	Adapter->GetInfluencedVertexData(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles, Verts);

	if(Verts.Num() > 0)
	{
		// Sort based on distance from brush centre
		Verts.Sort([=](const TPair<int32, FVector>& Vert0, const TPair<int32, FVector>& Vert1)
		{
			return (Vert0.Value - ComponentSpaceBrushPosition).SizeSquared() < (Vert1.Value - ComponentSpaceBrushPosition).SizeSquared();
		});

		// Fill operates on one vertex only, the closest
		int32 ChosenIndex = Verts[0].Key;

		// An expansion queue to handle the fill operation
		TQueue<int32> VertQueue;

		// Query values to account for the threshold around the selected value
		const float QueryValue = SharedPainter->GetPropertyValue(ChosenIndex);
		const float MinQueryValue = QueryValue - Settings->Threshold;
		const float MaxQueryValue = QueryValue + Settings->Threshold;

		// Set the selected vert to the new vaue and add it to the expansion queue
		SharedPainter->SetPropertyValue(ChosenIndex, Settings->FillValue);
		VertQueue.Enqueue(ChosenIndex);

		int32 CurrIndex = 0;
		while(!VertQueue.IsEmpty())
		{
			// Remove the current vert from the queue
			VertQueue.Dequeue(CurrIndex);

			// Grab the neighbors of the current vertex
			const TArray<int32>* Neighbors = Adapter->GetVertexNeighbors(CurrIndex);

			if(Neighbors && Neighbors->Num() > 0)
			{
				for(int32 NeighborIndex : (*Neighbors))
				{
					// For each neighbor, get its current value and if it's not the correct final
					// value, set it and then add to the queue for expansion on the next loop
					const float NeighborValue = SharedPainter->GetPropertyValue(NeighborIndex);
					if(NeighborValue != Settings->FillValue && NeighborValue >= MinQueryValue && NeighborValue <= MaxQueryValue)
					{
						SharedPainter->SetPropertyValue(NeighborIndex, Settings->FillValue);
						VertQueue.Enqueue(NeighborIndex);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
