// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothPaintToolBase.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Math/Matrix.h"
#include "MeshPaintHelpers.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothPaintTools.generated.h"

class FClothPainter;
class FEditorViewportClient;
class FMeshPaintParameters;
class FPrimitiveDrawInterface;
class FSceneView;
class FUICommandList;
class FViewport;
class IDetailsView;
class IMeshPaintGeometryAdapter;
class UClothPainterSettings;
class USkeletalMeshComponent;

/** Unique settings for the Brush tool */
UCLASS()
class UClothPaintTool_BrushSettings : public UObject
{
	GENERATED_BODY()

public:

	UClothPaintTool_BrushSettings()
		: PaintValue(100.0f)
	{

	}

	/** Value to paint onto the mesh for this parameter */
	UPROPERTY(EditAnywhere, Category = ToolSettings, meta=(UIMin=0, UIMax=100, ClampMin=0, ClampMax=100000))
	float PaintValue;
};


/** Standard brush tool for painting onto the mesh */
class FClothPaintTool_Brush : public FClothPaintToolBase
{
public:

	FClothPaintTool_Brush(TWeakPtr<FClothPainter> InPainter)
		: FClothPaintToolBase(InPainter)
		, Settings(nullptr)
	{

	}

	virtual ~FClothPaintTool_Brush();

	/** FClothPaintToolBase interface */
	virtual FText GetDisplayName() const override;
	virtual FPerVertexPaintAction GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings) override;
	virtual UObject* GetSettingsObject() override;
	virtual bool HasValueRange() override;
	virtual void GetValueRange(float& OutRangeMin, float& OutRangeMax) override;
	/** End FClothPaintToolBase interface */

protected:

	/** The settings object shown in the details panel */
	UClothPaintTool_BrushSettings* Settings;

	/** Called when the paint action is applied */
	void PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix);

};

/** Unique settings for the Gradient tool */
UCLASS()
class UClothPaintTool_GradientSettings : public UObject
{

	GENERATED_BODY()

public:

	UClothPaintTool_GradientSettings()
		: GradientStartValue(0.0f)
		, GradientEndValue(100.0f)
		, bUseRegularBrush(false)
	{

	}

	/** Value of the gradient at the start points */
	UPROPERTY(EditAnywhere, Category = ToolSettings, meta=(UIMin=0, UIMax=100, ClampMin=0, ClampMax=100000))
	float GradientStartValue;

	/** Value of the gradient at the end points */
	UPROPERTY(EditAnywhere, Category = ToolSettings, meta=(UIMin=0, UIMax=100, ClampMin=0, ClampMax=100000))
	float GradientEndValue;

	/** Enables the painting of selected points using a brush rather than just a point */
	UPROPERTY(EditAnywhere, Category = ToolSettings)
	bool bUseRegularBrush;
};

/** 
 * Gradient tool - Allows the user to select begin and end points to apply a gradient to.
 * Pressing Enter will move from selecting begin points, to selecting end points and finally
 * applying the operation to the mesh
 */
class FClothPaintTool_Gradient : public FClothPaintToolBase
{
public:

	FClothPaintTool_Gradient(TWeakPtr<FClothPainter> InPainter)
		: FClothPaintToolBase(InPainter)
		, bSelectingBeginPoints(true)
		, Settings(nullptr)
	{

	}

	virtual ~FClothPaintTool_Gradient();

	/* FClothPaintToolBase interface */
	virtual FText GetDisplayName() const override;
	virtual bool InputKey(IMeshPaintGeometryAdapter* Adapter, FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	virtual FPerVertexPaintAction GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings) override;
	virtual bool IsPerVertex() const override;
	virtual void Render(USkeletalMeshComponent* InComponent, IMeshPaintGeometryAdapter* InAdapter, const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool ShouldRenderInteractors() const override;
	virtual UObject* GetSettingsObject() override;
	virtual void Activate(TWeakPtr<FUICommandList> InCommands) override;
	virtual void Deactivate(TWeakPtr<FUICommandList> InCommands) override;
	virtual void OnMeshChanged() override;
	virtual bool HasValueRange() override;
	virtual void GetValueRange(float& OutRangeMin, float& OutRangeMax) override;
	/* End FClothPaintToolBase interface */
	
protected:

	/** Whether we're selecting the start points or end points */
	bool bSelectingBeginPoints;

	/** List of vert indices to consider the start of the gradient */
	TArray<int32> GradientStartIndices;

	/** List of vert indices to consider the end of the gradient */
	TArray<int32> GradientEndIndices;

	/** The settings object shown in the details panel */
	UClothPaintTool_GradientSettings* Settings;

	/** Called when the paint action is applied */
	void PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix);

	/** Applies the gradient to the currently selected points */
	void ApplyGradient();

	/** Whether we can currently apply a gradient */
	bool CanApplyGradient();

};

/** Unique settings for the smoothing tool */
UCLASS()
class UClothPaintTool_SmoothSettings : public UObject
{
	GENERATED_BODY()

public:

	UClothPaintTool_SmoothSettings()
		: Strength(0.2f)
	{

	}

	/** Strength of the smoothing effect */
	UPROPERTY(EditAnywhere, Category = ToolSettings, meta=(UIMin="0.0", UIMax="1.0", ClampMin="0.0", ClampMax="1.0"))
	float Strength;
};

/** 
 * Smoothing tool, applies a blur similar to a box blur (even distribution of neighbors)
 * Modulated by strength from settings object
 */
class FClothPaintTool_Smooth : public FClothPaintToolBase
{
public:

	FClothPaintTool_Smooth(TWeakPtr<FClothPainter> InPainter)
		: FClothPaintToolBase(InPainter)
		, Settings(nullptr)
	{

	}

	virtual ~FClothPaintTool_Smooth();

	virtual FPerVertexPaintAction GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings) override;
	virtual FText GetDisplayName() const override;
	virtual UObject* GetSettingsObject() override;
	virtual void RegisterSettingsObjectCustomizations(IDetailsView* InDetailsView) override;
	virtual bool IsPerVertex() const override;

	/** Given a set of vertex indices, apply the smooth operation over the set */
	void SmoothVertices(const TSet<int32> &InfluencedVertices, TSharedPtr<FClothPainter> SharedPainter);

protected:

	void PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix);

	UClothPaintTool_SmoothSettings* Settings;
};

/** Unique settings for the fill tool */
UCLASS()
class UClothPaintTool_FillSettings : public UObject
{
	GENERATED_BODY()
public:

	UClothPaintTool_FillSettings()
		: Threshold(0.0f), FillValue(100.0f)
	{}

	/** Threshold for fill operation, will keep filling until sampled verts aren't within this range of the original vertex */
	UPROPERTY(EditAnywhere, Category = ToolSettings, meta = (UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 100000))
	float Threshold;

	/** The value to fill all selected verts to */
	UPROPERTY(EditAnywhere, Category = ToolSettings, meta = (UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 100000))
	float FillValue;
};

/** 
 * Basic fill tool with thresholding for changing the parameter values for a large area of similar cloth
 */
class FClothPaintTool_Fill : public FClothPaintToolBase
{
public:

	FClothPaintTool_Fill(TWeakPtr<FClothPainter> InPainter)
		: FClothPaintToolBase(InPainter)
		, Settings(nullptr)
		, QueryRadius(20.0f) // No brush available for fill, gives decent range to get closest point to cursor.
	{

	}

	virtual ~FClothPaintTool_Fill();

	/* FClothPaintToolBase interface */
	virtual FPerVertexPaintAction GetPaintAction(const FMeshPaintParameters& InPaintParams, UClothPainterSettings* InPainterSettings) override;
	virtual FText GetDisplayName() const override;
	virtual UObject* GetSettingsObject() override;
	virtual bool IsPerVertex() const override;
	virtual bool ShouldRenderInteractors() const override;
	virtual void Render(USkeletalMeshComponent* InComponent, IMeshPaintGeometryAdapter* InAdapter, const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HasValueRange() override;
	virtual void GetValueRange(float& OutRangeMin, float& OutRangeMax) override;
	/* End FClothPaintToolBase interface */

protected:

	/** Called to apply the fill action */
	void PaintAction(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMatrix InverseBrushMatrix);

	/** Settings for the paint operation */
	UClothPaintTool_FillSettings* Settings;

	/** Radius to query for points (will choose nearest point in this set) */
	float QueryRadius;
};
