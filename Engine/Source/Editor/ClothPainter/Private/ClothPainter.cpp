// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPainter.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "ClothLODData.h"
#include "ClothMeshAdapter.h"
#include "ClothPaintSettings.h"
#include "ClothPaintToolBase.h"
#include "ClothPaintTools.h"
#include "ClothingAsset.h"
#include "ClothingAssetBase.h"
#include "CollisionQueryParams.h"
#include "ComponentReregisterContext.h"
#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "Engine/EngineTypes.h"
#include "Engine/NetSerialization.h"
#include "Engine/SkeletalMesh.h"
#include "EngineDefines.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/PlatformCrt.h"
#include "IMeshPaintGeometryAdapter.h"
#include "Internationalization/Internationalization.h"
#include "Math/Color.h"
#include "Math/Matrix.h"
#include "Math/NumericLimits.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintSettings.h"
#include "Misc/Guid.h"
#include "PointWeightMap.h"
#include "SClothPaintWidget.h"
#include "Stats/Stats.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FPrimitiveDrawInterface;
class FSceneView;
class UMeshComponent;

#define LOCTEXT_NAMESPACE "ClothPainter"

namespace ClothPaintConstants
{
	const float HoverQueryRadius = 50.0f;
}

FClothPainter::FClothPainter()
	: IMeshPainter()
	, SkeletalMeshComponent(nullptr)
	, PaintSettings(nullptr)
	, BrushSettings(nullptr)
{
	VertexPointColor = FLinearColor::White;
	WidgetLineThickness = .0f;
	bShouldSimulate = false;
	bShowHiddenVerts = false;
}

FClothPainter::~FClothPainter()
{
	SkeletalMeshComponent->SetMeshSectionVisibilityForCloth(SkeletalMeshComponent->SelectedClothingGuidForPainting, true);

	// Cancel rendering of the paint proxy
	SkeletalMeshComponent->SelectedClothingGuidForPainting = FGuid();
}

void FClothPainter::Init()
{
	// Brush settings share their values by getting/setting per-property values in the config via Constructor/PostEditChange.
	BrushSettings = NewObject<UPaintBrushSettings>(GetTransientPackage());
	BrushSettings->AddToRoot();
	BrushSettings->bOnlyFrontFacingTriangles = false;
	PaintSettings = GetMutableDefault<UClothPainterSettings>();
	PaintSettings->LoadConfig();

	CommandList = MakeShareable(new FUICommandList);

	Tools.Add(MakeShared<FClothPaintTool_Brush>(AsShared()));
	Tools.Add(MakeShared<FClothPaintTool_Gradient>(AsShared()));
	Tools.Add(MakeShared<FClothPaintTool_Smooth>(AsShared()));
	Tools.Add(MakeShared<FClothPaintTool_Fill>(AsShared()));

	SelectedTool = Tools[0];
	SelectedTool->Activate(CommandList);

	Widget = SNew(SClothPaintWidget, this);
}

bool FClothPainter::PaintInternal(const FVector& InCameraOrigin, const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintAction PaintAction, float PaintStrength)
{
	bool bApplied = false;

	if(SkeletalMeshComponent->SelectedClothingGuidForPainting.IsValid() && !bShouldSimulate)
	{
		USkeletalMesh* SkelMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();

		for (const TPair<FVector, FVector>& Ray : Rays)
		{
			const FVector& InRayOrigin = Ray.Key;
			const FVector& InRayDirection = Ray.Value;
			const FHitResult& HitResult = GetHitResult(InRayOrigin, InRayDirection);

			if (HitResult.bBlockingHit)
			{
				// Generic per-vertex painting operations
				if (!IsPainting())
				{
					BeginTransaction(LOCTEXT("MeshPaint", "Painting Cloth Property Values"));
					bArePainting = true;
					Adapter->PreEdit();
				}

				const FMeshPaintParameters Parameters = CreatePaintParameters(HitResult, InCameraOrigin, InRayOrigin, InRayDirection, PaintStrength);

				FPerVertexPaintActionArgs Args;
				Args.Adapter = Adapter.Get();
				Args.CameraPosition = InCameraOrigin;
				Args.HitResult = HitResult;
				Args.BrushSettings = GetBrushSettings();
				Args.Action = PaintAction;

				if (SelectedTool->IsPerVertex())
				{
					bApplied |= MeshPaintHelpers::ApplyPerVertexPaintAction(Args, GetPaintAction(Parameters));
				}
				else
				{
					bApplied = true;
					GetPaintAction(Parameters).ExecuteIfBound(Args, INDEX_NONE);
				}
			}
		}
	}

	return bApplied;
}

FPerVertexPaintAction FClothPainter::GetPaintAction(const FMeshPaintParameters& InPaintParams)
{
	if(SelectedTool.IsValid())
	{
		return SelectedTool->GetPaintAction(InPaintParams, PaintSettings);
	}

	return FPerVertexPaintAction();
}

void FClothPainter::SetTool(TSharedPtr<FClothPaintToolBase> InTool)
{
	if(InTool.IsValid() && Tools.Contains(InTool))
	{
		if(SelectedTool.IsValid())
		{
			SelectedTool->Deactivate(CommandList);
		}

		SelectedTool = InTool;
		SelectedTool->Activate(CommandList);
	}
}

void FClothPainter::SetSkeletalMeshComponent(UDebugSkelMeshComponent* InSkeletalMeshComponent)
{
	TSharedPtr<FClothMeshPaintAdapter> Result = MakeShareable(new FClothMeshPaintAdapter());
	Result->Construct(InSkeletalMeshComponent, 0);
	Adapter = Result;

	SkeletalMeshComponent = InSkeletalMeshComponent;

	RefreshClothingAssets();

	if (Widget.IsValid())
	{
		Widget->OnRefresh();
	}
}

USkeletalMesh* FClothPainter::GetSkeletalMesh() const
{
	if(SkeletalMeshComponent)
	{
		return SkeletalMeshComponent->GetSkeletalMeshAsset();
	}

	return nullptr;
}

void FClothPainter::RefreshClothingAssets()
{
	if(!PaintSettings || !SkeletalMeshComponent)
	{
		return;
	}

	PaintSettings->ClothingAssets.Reset();

	if(USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
	{
		for(UClothingAssetBase* BaseClothingAsset : Mesh->GetMeshClothingAssets())
		{
			if(UClothingAssetCommon* ActualAsset = Cast<UClothingAssetCommon>(BaseClothingAsset))
			{
				PaintSettings->ClothingAssets.AddUnique(ActualAsset);
			}
		}
	}
}

void FClothPainter::EnterPaintMode()
{
	Reset();

	if(SkeletalMeshComponent)
	{
		HoveredTextCallbackHandle = SkeletalMeshComponent->RegisterExtendedViewportTextDelegate(FGetExtendedViewportText::CreateSP(this, &FClothPainter::GetViewportText));
	}
}

void FClothPainter::ExitPaintMode()
{
	if(SkeletalMeshComponent)
	{
		SkeletalMeshComponent->UnregisterExtendedViewportTextDelegate(HoveredTextCallbackHandle);
	}

	// Remove reference to asset so it can be GC if necessary
	if (PaintSettings)
	{
		PaintSettings->ClothingAssets.Reset();
		PaintSettings->SaveConfig();
	}
}

void FClothPainter::RecalculateAutoViewRange()
{
	if(!Adapter.IsValid())
	{
		return;
	}

	TSharedPtr<FClothMeshPaintAdapter> ClothAdapter = StaticCastSharedPtr<FClothMeshPaintAdapter>(Adapter);
	FPointWeightMap* CurrentMask = ClothAdapter->GetCurrentMask();

	if(UClothPainterSettings* PainterSettings = Cast<UClothPainterSettings>(GetPainterSettings()))
	{
		if(PainterSettings->bAutoViewRange && CurrentMask)
		{
			float MinValue = MAX_flt;
			float MaxValue = -MinValue;
			CurrentMask->CalcRanges(MinValue, MaxValue);

			PainterSettings->AutoCalculatedViewMin = MinValue;
			PainterSettings->AutoCalculatedViewMax = MaxValue;
		}
		else
		{
			PainterSettings->AutoCalculatedViewMin = 0.0f;
			PainterSettings->AutoCalculatedViewMax = 0.0f;
		}
	}
}

void FClothPainter::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	IMeshPainter::Tick(ViewportClient, DeltaTime);

	SkeletalMeshComponent->MinClothPropertyView = PaintSettings->GetViewMin();
	SkeletalMeshComponent->MaxClothPropertyView = PaintSettings->GetViewMax();

	if(SelectedTool.IsValid() && PaintSettings->bAutoViewRange)
	{
		if(SelectedTool->HasValueRange())
		{
			float ToolMinRange = SkeletalMeshComponent->MinClothPropertyView;
			float ToolMaxRange = SkeletalMeshComponent->MaxClothPropertyView;

			SelectedTool->GetValueRange(ToolMinRange, ToolMaxRange);

			SkeletalMeshComponent->MinClothPropertyView = FMath::Min(SkeletalMeshComponent->MinClothPropertyView, ToolMinRange);
			SkeletalMeshComponent->MaxClothPropertyView = FMath::Max(SkeletalMeshComponent->MaxClothPropertyView, ToolMaxRange);
		}
	}

	SkeletalMeshComponent->bClothFlipNormal = PaintSettings->bFlipNormal;
	SkeletalMeshComponent->bClothCullBackface = PaintSettings->bCullBackface;
	SkeletalMeshComponent->ClothMeshOpacity = PaintSettings->Opacity;

	if ((bShouldSimulate && SkeletalMeshComponent->bDisableClothSimulation) || (!bShouldSimulate && !SkeletalMeshComponent->bDisableClothSimulation))
	{
		if(bShouldSimulate)
		{
			// Need to re-apply our masks here, as they have likely been edited
			for(UClothingAssetCommon* Asset : PaintSettings->ClothingAssets)
			{
				if(Asset)
				{
					constexpr bool bUpdateFixedVertData = true;  // There's Currently no way of telling whether the MaxDistance mask has been edited
					constexpr bool bInvalidateDerivedDataCache = false;  // No need to rebuild the DDC while previewing

					Asset->ApplyParameterMasks(bUpdateFixedVertData, bInvalidateDerivedDataCache);
				}
			}
		}

		FComponentReregisterContext ReregisterContext(SkeletalMeshComponent);
		SkeletalMeshComponent->bDisableClothSimulation = !bShouldSimulate;
		SkeletalMeshComponent->bShowClothData = !bShouldSimulate;
		SkeletalMeshComponent->SetMeshSectionVisibilityForCloth(SkeletalMeshComponent->SelectedClothingGuidForPainting, bShouldSimulate);
		ViewportClient->Invalidate();
	}

	
	// We always want up to date CPU skinned verts, so each tick we reinitialize the adapter
	if(Adapter.IsValid())
	{
		Adapter->Initialize();
	}
}

void FClothPainter::FinishPainting()
{
	if (IsPainting())
	{		
		EndTransaction();
		Adapter->PostEdit();

		/** If necessary, recalculate view ranges when set to auto mode */
		RecalculateAutoViewRange();
	}

	bArePainting = false;
}

void FClothPainter::Reset()
{	
	if(Widget.IsValid())
	{
		Widget->Reset();
	}

	bArePainting = false;
	SkeletalMeshComponent->SetMeshSectionVisibilityForCloth(SkeletalMeshComponent->SelectedClothingGuidForPainting, true);
	SkeletalMeshComponent->SelectedClothingGuidForPainting = FGuid();
	bShouldSimulate = false;
}

TSharedPtr<IMeshPaintGeometryAdapter> FClothPainter::GetMeshAdapterForComponent(const UMeshComponent* Component)
{
	if (Component == SkeletalMeshComponent)
	{
		return Adapter;
	}

	return nullptr;
}

void FClothPainter::AddReferencedObjects(FReferenceCollector& Collector)
{	
	Collector.AddReferencedObject(SkeletalMeshComponent);
	Collector.AddReferencedObject(BrushSettings);
	Collector.AddReferencedObject(PaintSettings);
}

UPaintBrushSettings* FClothPainter::GetBrushSettings()
{
	return BrushSettings;
}

UMeshPaintSettings* FClothPainter::GetPainterSettings()
{
	return PaintSettings;
}

TSharedPtr<class SWidget> FClothPainter::GetWidget()
{
	return Widget;
}

const FHitResult FClothPainter::GetHitResult(const FVector& Origin, const FVector& Direction)
{
	FHitResult HitResult(1.0f);
	const FVector TraceStart(Origin);
	const FVector TraceEnd(Origin + Direction * HALF_WORLD_MAX);

	if (Adapter.IsValid())
	{
		Adapter->LineTraceComponent(HitResult, TraceStart, TraceEnd, FCollisionQueryParams(SCENE_QUERY_STAT(FClothPainter_GetHitResult), true));
	}
	
	return HitResult;
}

void FClothPainter::Refresh()
{
	RefreshClothingAssets();
	if(Widget.IsValid())
	{
		Widget->OnRefresh();
	}
}

void FClothPainter::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if(SelectedTool.IsValid() && SelectedTool->ShouldRenderInteractors() && !bShouldSimulate)
	{
		RenderInteractors(View, Viewport, PDI, true, SDPG_Foreground);
	}

	if(Adapter.IsValid() && SkeletalMeshComponent)
	{
		TSharedPtr<FClothMeshPaintAdapter> ClothAdapter = StaticCastSharedPtr<FClothMeshPaintAdapter>(Adapter);

		TArray<MeshPaintHelpers::FPaintRay> PaintRays;
		MeshPaintHelpers::RetrieveViewportPaintRays(View, Viewport, PDI, PaintRays);

		bool bFoundValue = false;
		float Value = 0.0f;

		for(const MeshPaintHelpers::FPaintRay& PaintRay : PaintRays)
		{
			const FHitResult& HitResult = GetHitResult(PaintRay.RayStart, PaintRay.RayDirection);

			if(HitResult.Component == SkeletalMeshComponent.Get())
			{
				const FMatrix ComponentToWorldMatrix = SkeletalMeshComponent->GetComponentTransform().ToMatrixWithScale();
				const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(PaintRay.CameraLocation));
				const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitResult.Location));
				const float ComponentSpaceBrushRadius = ClothPaintConstants::HoverQueryRadius;
				const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

				TArray<TPair<int32, FVector>> VertexData;
				ClothAdapter->GetInfluencedVertexData(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, BrushSettings->bOnlyFrontFacingTriangles, VertexData);
				
				FPointWeightMap* CurrentMask = ClothAdapter->GetCurrentMask();

				if(CurrentMask && VertexData.Num() > 0)
				{
					int32 ClosestIndex = INDEX_NONE;
					float ClosestDistanceSq = MAX_flt;
					int32 NumVertsFound = VertexData.Num();
					for(int32 CurrIndex = 0; CurrIndex < NumVertsFound; ++CurrIndex)
					{
						const float DistSq = (VertexData[CurrIndex].Value - ComponentSpaceBrushPosition).SizeSquared();

						if(DistSq < ClosestDistanceSq)
						{
							ClosestDistanceSq = DistSq;
							ClosestIndex = CurrIndex;
						}
					}

					if(ClosestIndex != INDEX_NONE)
					{
						TPair<int32, FVector>& Nearest = VertexData[ClosestIndex];
						bFoundValue = true;
						Value = CurrentMask->GetValue(Nearest.Key);

						break;
					}
				}
			}
		}

		if(PaintRays.Num() > 0)
		{
			FText BaseText = LOCTEXT("ClothPaintViewportValueText", "Cloth Value: {0}");

			if(bFoundValue)
			{
				FNumberFormattingOptions Options;
				Options.MinimumFractionalDigits = 3;
				Options.MaximumFractionalDigits = 3;

				CachedHoveredClothValueText = FText::Format(BaseText, FText::AsNumber(Value, &Options));
			}
			else
			{
				CachedHoveredClothValueText = FText::Format(BaseText, LOCTEXT("ClothPaintViewportNotApplicable", "N/A"));
			}
		}
	}
	
	const ESceneDepthPriorityGroup DepthPriority = bShowHiddenVerts ? SDPG_Foreground : SDPG_World;
	// Render simulation mesh vertices if not simulating
	if(SkeletalMeshComponent)
	{
		if(!bShouldSimulate)
		{
			if(SelectedTool.IsValid())
			{
				SelectedTool->Render(SkeletalMeshComponent, Adapter.Get(), View, Viewport, PDI);
			}
		}
	}
	
	bShouldSimulate = Viewport->KeyState(EKeys::H);
	bShowHiddenVerts = Viewport->KeyState(EKeys::J);
}

bool FClothPainter::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
	bool bHandled = IMeshPainter::InputKey(InViewportClient, InViewport, InKey, InEvent);

	if(SelectedTool.IsValid())
	{
		if(CommandList->ProcessCommandBindings(InKey, FSlateApplication::Get().GetModifierKeys(), InEvent == IE_Repeat))
		{
			bHandled = true;
		}
		else
		{
			// Handle non-action based key actions (holds etc.)
			bHandled |= SelectedTool->InputKey(Adapter.Get(), InViewportClient, InViewport, InKey, InEvent);
		}
	}

	return bHandled;
}

FMeshPaintParameters FClothPainter::CreatePaintParameters(const FHitResult& HitResult, const FVector& InCameraOrigin, const FVector& InRayOrigin, const FVector& InRayDirection, float PaintStrength)
{
	const float BrushStrength = BrushSettings->BrushStrength *  BrushSettings->BrushStrength * PaintStrength;

	const float BrushRadius = BrushSettings->GetBrushRadius();
	const float BrushDepth = BrushRadius * .5f;

	FVector BrushXAxis, BrushYAxis;
	HitResult.Normal.FindBestAxisVectors(BrushXAxis, BrushYAxis);
	// Display settings
	const float VisualBiasDistance = 0.15f;
	const FVector BrushVisualPosition = HitResult.Location + HitResult.Normal * VisualBiasDistance;

	FMeshPaintParameters Params;
	{
		Params.BrushPosition = HitResult.Location;
		
		Params.SquaredBrushRadius = BrushRadius * BrushRadius;
		Params.BrushRadialFalloffRange = BrushSettings->BrushFalloffAmount * BrushRadius;
		Params.InnerBrushRadius = BrushRadius - Params.BrushRadialFalloffRange;
		Params.BrushDepth = BrushDepth;
		Params.BrushDepthFalloffRange = BrushSettings->BrushFalloffAmount * BrushDepth;
		Params.InnerBrushDepth = BrushDepth - Params.BrushDepthFalloffRange;
		Params.BrushStrength = BrushStrength;
		Params.BrushNormal = HitResult.Normal;
		Params.BrushToWorldMatrix = FMatrix(BrushXAxis, BrushYAxis, Params.BrushNormal, Params.BrushPosition);
		Params.InverseBrushToWorldMatrix = Params.BrushToWorldMatrix.InverseFast();
	}

	return Params;
}

float FClothPainter::GetPropertyValue(int32 VertexIndex)
{
	FClothMeshPaintAdapter* ClothAdapter = (FClothMeshPaintAdapter*)Adapter.Get();

	if(FPointWeightMap* Mask = ClothAdapter->GetCurrentMask())
	{
		return Mask->GetValue(VertexIndex);
	}

	return 0.0f;
}

void FClothPainter::SetPropertyValue(int32 VertexIndex, const float Value)
{
	FClothMeshPaintAdapter* ClothAdapter = (FClothMeshPaintAdapter*)Adapter.Get();

	if(FPointWeightMap* Mask = ClothAdapter->GetCurrentMask())
	{
		Mask->SetValue(VertexIndex, Value);
	}
}

void FClothPainter::OnAssetSelectionChanged(UClothingAssetCommon* InNewSelectedAsset, int32 InAssetLod, int32 InMaskIndex)
{
	TSharedPtr<FClothMeshPaintAdapter> ClothAdapter = StaticCastSharedPtr<FClothMeshPaintAdapter>(Adapter);
	if(ClothAdapter.IsValid() && InNewSelectedAsset && InNewSelectedAsset->IsValidLod(InAssetLod))
	{
		// Validate the incoming parameters, to make sure we only set a selection if we're going
		// to get a valid paintable surface
		if(InNewSelectedAsset->LodData.IsValidIndex(InAssetLod) &&
			 InNewSelectedAsset->LodData[InAssetLod].PointWeightMaps.IsValidIndex(InMaskIndex))
		{
			const FGuid NewGuid = InNewSelectedAsset->GetAssetGuid();
			SkeletalMeshComponent->SetMeshSectionVisibilityForCloth(SkeletalMeshComponent->SelectedClothingGuidForPainting, true);
			SkeletalMeshComponent->SetMeshSectionVisibilityForCloth(NewGuid, false);

			SkeletalMeshComponent->bDisableClothSimulation = true;
			SkeletalMeshComponent->bShowClothData = true;
			SkeletalMeshComponent->SelectedClothingGuidForPainting = NewGuid;
			SkeletalMeshComponent->SelectedClothingLodForPainting = InAssetLod;
			SkeletalMeshComponent->SelectedClothingLodMaskForPainting = InMaskIndex;
			SkeletalMeshComponent->RefreshSelectedClothingSkinnedPositions();

			ClothAdapter->SetSelectedClothingAsset(NewGuid, InAssetLod, InMaskIndex);

			for(TSharedPtr<FClothPaintToolBase> Tool : Tools)
			{
				Tool->OnMeshChanged();
			}
			return;
		}
	}

	// Nothing to paint
	SkeletalMeshComponent->SetMeshSectionVisibilityForCloth(SkeletalMeshComponent->SelectedClothingGuidForPainting, true);
	SkeletalMeshComponent->SelectedClothingGuidForPainting = FGuid();
	SkeletalMeshComponent->bDisableClothSimulation = false;
}

#undef LOCTEXT_NAMESPACE // "ClothPainter"
