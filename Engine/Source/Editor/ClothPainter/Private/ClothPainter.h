// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/EngineBaseTypes.h"
#include "IMeshPainter.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathSSE.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintTypes.h"
#include "Templates/SharedPointer.h"

class AActor;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FReferenceCollector;
class FSceneView;
class FUICommandList;
class FViewport;
class IMeshPaintGeometryAdapter;
class SClothPaintWidget;
class UClothPainterSettings;
class UClothingAssetCommon;
class UDebugSkelMeshComponent;
class UMeshComponent;
class UMeshPaintSettings;
class UPaintBrushSettings;
class USkeletalMesh;
struct FHitResult;

enum class EPaintableClothProperty;
class FClothPaintToolBase;

class FClothPainter : public IMeshPainter, public TSharedFromThis<FClothPainter>
{
public:

	FClothPainter();
	~FClothPainter();

	void Init();

protected:

	virtual bool PaintInternal(const FVector& InCameraOrigin, const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintAction PaintAction, float PaintStrength) override;

public:

	/** Mode transition, called by the paint mode as painting is enabled and disabled */
	void EnterPaintMode();
	void ExitPaintMode();

	/** Called to update the auto view min/max values  */
	void RecalculateAutoViewRange();

	/** IMeshPainter interface */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;	
	virtual void FinishPainting() override;
	virtual void ActorSelected(AActor* Actor) override {};
	virtual void ActorDeselected(AActor* Actor) override {};
	virtual void Reset() override;
	virtual TSharedPtr<IMeshPaintGeometryAdapter> GetMeshAdapterForComponent(const UMeshComponent* Component) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual UPaintBrushSettings* GetBrushSettings() override;
	virtual UMeshPaintSettings* GetPainterSettings() override;
	virtual TSharedPtr<class SWidget> GetWidget() override;
	virtual const FHitResult GetHitResult(const FVector& Origin, const FVector& Direction) override;
	virtual void Refresh() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	/** End IMeshPainter interface */

	/** Sets the debug skeletal mesh to which we should currently paint */
	void SetSkeletalMeshComponent(UDebugSkelMeshComponent* SkeletalMeshComponent);

	/** Gets the skeletal mesh of the current skeletal mesh component */
	USkeletalMesh* GetSkeletalMesh() const;

	/** Creates paint parameters for the current setup */
	FMeshPaintParameters CreatePaintParameters(const struct FHitResult& HitResult, const FVector& InCameraOrigin, const FVector& InRayOrigin, const FVector& InRayDirection, float PaintStrength);

	/** Retrieves the property value from the Cloth asset for the given EPaintableClothProperty */
	float GetPropertyValue(int32 VertexIndex);
	/** Sets the EPaintableClothProperty property within the Clothing asset to Value */
	void SetPropertyValue(int32 VertexIndex, const float Value);

	/** Some complex clothing tools (gradients) require the ability to override these flags in different ways */
	void SetIsPainting(bool bInPainting) { bArePainting = bInPainting; }

	/** Get the selected paint tool */
	const TSharedPtr<FClothPaintToolBase> GetSelectedTool() const { return SelectedTool; }

	/** Gets the current geometry adapter */
	TSharedPtr<IMeshPaintGeometryAdapter> GetAdapter() const { return Adapter; }

	/** When a different clothing asset is selected in the UI the painter should refresh the adapter */
	void OnAssetSelectionChanged(UClothingAssetCommon* InNewSelectedAsset, int32 InAssetLod, int32 MaskIndex);
	void OnAssetMaskSelectionChanged()
	{};

	/** Returns custom text to display in the skeletal mesh editor viewport */
	FText GetViewportText() const { return CachedHoveredClothValueText; }

protected:

	/** Rebuild the list of editable clothing assets from the current mesh */
	void RefreshClothingAssets();

	/** Get the action defined by the selected tool that we should run when we paint */
	FPerVertexPaintAction GetPaintAction(const FMeshPaintParameters& InPaintParams);

	/** 
	 * Sets the currently selected paint tool
	 * NOTE: InTool *must* have been registered by adding it to the Tools array
	 */
	void SetTool(TSharedPtr<FClothPaintToolBase> InTool);

	/** Current adapter used to paint the clothing properties */
	TSharedPtr<IMeshPaintGeometryAdapter> Adapter;	
	/** Debug skeletal mesh to which painting should be applied */
	TObjectPtr<UDebugSkelMeshComponent> SkeletalMeshComponent;
	/** Widget used to represent the state/functionality of the painter */
	TSharedPtr<SClothPaintWidget> Widget;
	/** Cloth paint settings instance */
	TObjectPtr<UClothPainterSettings> PaintSettings;
	/** Cloth brush settings instance */
	TObjectPtr<UPaintBrushSettings> BrushSettings;

	/** Flag whether or not the simulation should run */
	bool bShouldSimulate;
	/** Flag to render (hidden) sim verts during gradient painting */
	bool bShowHiddenVerts;

	/** The currently selected painting tool */
	TSharedPtr<FClothPaintToolBase> SelectedTool;

	/** List of currently registered paint tools */
	TArray<TSharedPtr<FClothPaintToolBase>> Tools;

	/** List of commands for the painter, tools can bind to this in activate */
	TSharedPtr<FUICommandList> CommandList;

	FText CachedHoveredClothValueText;

	FDelegateHandle HoveredTextCallbackHandle;

	/** Our customization class can access private painter state */
	friend class FClothPaintSettingsCustomization;
};