// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPainterCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "ClothPaintCommands"

void FClothPainterCommands::RegisterCommands()
{
	UI_COMMAND(TogglePaintMode, "Enable Cloth Paint", "Toggles between selection and clothing paint modes.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

const FClothPainterCommands& FClothPainterCommands::Get()
{
	return TCommands<FClothPainterCommands>::Get();
}

#undef LOCTEXT_NAMESPACE
