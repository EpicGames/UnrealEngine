// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FClothPainterCommands : public TCommands<FClothPainterCommands>
{
public:
	FClothPainterCommands()
		: TCommands<FClothPainterCommands>(TEXT("ClothPainterTools"), NSLOCTEXT("Contexts", "ClothPainter", "Cloth Painter"), NAME_None, FAppStyle::GetAppStyleSetName())
	{

	}

	virtual void RegisterCommands() override;
	static const FClothPainterCommands& Get();

	/** Clothing commands */
	TSharedPtr<FUICommandInfo> TogglePaintMode;
};