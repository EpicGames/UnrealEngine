// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPaintingModule.h"

#include "ClothPaintToolCommands.h"
#include "ClothPainterCommands.h"
#include "ClothingPaintEditMode.h"
#include "EditorModeManager.h"
#include "Delegates/Delegate.h"
#include "EditorModeRegistry.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISkeletalMeshEditor.h"
#include "ISkeletalMeshEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SClothPaintTab.h"
#include "SkeletalMeshToolMenuContext.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuOwner.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FAssetEditorToolkit;
class SWidget;

#define LOCTEXT_NAMESPACE "ClothPaintingModule"

const FName PaintModeID = "ClothPaintMode";

IMPLEMENT_MODULE(FClothPaintingModule, ClothPainter);

DECLARE_DELEGATE_OneParam(FOnToggleClothPaintMode, bool);

struct FClothPaintTabSummoner : public FWorkflowTabFactory
{
public:
	/** Tab ID name */
	static const FName TabName;

	FClothPaintTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(TabName, InHostingApp)
	{
		TabLabel = LOCTEXT("ClothPaintTabLabel", "Clothing");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SkeletalMesh");
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return SNew(SClothPaintTab).InHostingApp(HostingApp);
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("ClothPaintTabToolTip", "Tab for Painting Cloth properties");
	}

	static TSharedPtr<FWorkflowTabFactory> CreateFactory(TSharedPtr<FAssetEditorToolkit> InAssetEditor)
	{
		return MakeShareable(new FClothPaintTabSummoner(InAssetEditor));
	}

protected:

};
const FName FClothPaintTabSummoner::TabName = TEXT("ClothPainting");

void FClothPaintingModule::StartupModule()
{	
	SetupMode();

	// Register any commands for the cloth painter
	ClothPaintToolCommands::RegisterClothPaintToolCommands();
	FClothPainterCommands::Register();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FClothPaintingModule::RegisterMenus));

	if(!SkelMeshEditorExtenderHandle.IsValid())
	{
		ISkeletalMeshEditorModule& SkelMeshEditorModule = FModuleManager::Get().LoadModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
		TArray<ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender>& Extenders = SkelMeshEditorModule.GetAllSkeletalMeshEditorToolbarExtenders();
		
		Extenders.Add(ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender::CreateRaw(this, &FClothPaintingModule::ExtendSkelMeshEditorToolbar));
		SkelMeshEditorExtenderHandle = Extenders.Last().GetHandle();
	}
}

void FClothPaintingModule::SetupMode()
{
	// Add application mode extender
	Extender = FWorkflowApplicationModeExtender::CreateRaw(this, &FClothPaintingModule::ExtendApplicationMode);
	FWorkflowCentricApplication::GetModeExtenderList().Add(Extender);

	FEditorModeRegistry::Get().RegisterMode<FClothingPaintEditMode>(PaintModeID, LOCTEXT("ClothPaintEditMode", "Cloth Painting"), FSlateIcon(), false);
}

TSharedRef<FApplicationMode> FClothPaintingModule::ExtendApplicationMode(const FName ModeName, TSharedRef<FApplicationMode> InMode)
{
	// For skeleton and animation editor modes add our custom tab factory to it
	if (ModeName == TEXT("SkeletalMeshEditorMode"))
	{
		InMode->AddTabFactory(FCreateWorkflowTabFactory::CreateStatic(&FClothPaintTabSummoner::CreateFactory));
		RegisteredApplicationModes.Add(InMode);
	}
	
	return InMode;
}

TSharedRef<FExtender> FClothPaintingModule::ExtendSkelMeshEditorToolbar(const TSharedRef<FUICommandList> InCommandList, TSharedRef<ISkeletalMeshEditor> InSkeletalMeshEditor)
{
	// Add toolbar extender
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	TWeakPtr<ISkeletalMeshEditor> Ptr(InSkeletalMeshEditor);

	InCommandList->MapAction(FClothPainterCommands::Get().TogglePaintMode,
		FExecuteAction::CreateRaw(this, &FClothPaintingModule::OnToggleMode, Ptr),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &FClothPaintingModule::IsPaintModeActive, Ptr)
	);

	return ToolbarExtender.ToSharedRef();
}

void FClothPaintingModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* Toolbar = UToolMenus::Get()->ExtendMenu("AssetEditor.SkeletalMeshEditor.ToolBar");
	{
		FToolMenuSection& Section = Toolbar->FindOrAddSection("SkeletalMesh");
		Section.AddDynamicEntry("TogglePaintMode", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			USkeletalMeshToolMenuContext* Context = InSection.FindContext<USkeletalMeshToolMenuContext>();
			if (Context && Context->SkeletalMeshEditor.IsValid())
			{
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
					FClothPainterCommands::Get().TogglePaintMode,
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &FClothPaintingModule::GetPaintToolsButtonText, Context->SkeletalMeshEditor)),
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &FClothPaintingModule::GetPaintToolsButtonToolTip, Context->SkeletalMeshEditor)),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MeshPaint.Brush")
				));	
			}
		}));
	}
}

FText FClothPaintingModule::GetPaintToolsButtonText(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const
{
	if(IsPaintModeActive(InSkeletalMeshEditor))
	{
		return LOCTEXT("ToggleButton_Deactivate", "Deactivate Cloth Paint");
	}

	return LOCTEXT("ToggleButton_Activate", "Activate Cloth Paint");
}

FText FClothPaintingModule::GetPaintToolsButtonToolTip(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const
{
	if(IsPaintModeActive(InSkeletalMeshEditor))
	{
		return LOCTEXT("ToggleButton_Deactivate_ToolTip", "Deactivate the cloth paint tool, and go back to the current selection mode.");
	}

	return LOCTEXT("ToggleButton_Activate_ToolTip", "Activate the cloth paint tool, and open the Clothing window to allow selection of the clothing assets and of their paint targets.");
}

bool FClothPaintingModule::IsPaintModeActive(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const
{
	if (!GetActiveClothTab(InSkeletalMeshEditor, false /* don't invoke tab*/).IsValid())
	{
		// haven't initialized UI yet
		return false;
	}
		
	TSharedPtr<ISkeletalMeshEditor> SkeletalMeshEditor = InSkeletalMeshEditor.Pin();
	return SkeletalMeshEditor.IsValid() && SkeletalMeshEditor->GetEditorModeManager().IsModeActive(PaintModeID);
}

void FClothPaintingModule::OnToggleMode(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const
{
	ISkeletalMeshEditor* SkeletalMeshEditor = InSkeletalMeshEditor.Pin().Get();
	FEditorModeTools& ModeManager = SkeletalMeshEditor->GetEditorModeManager();

	if (!IsPaintModeActive(InSkeletalMeshEditor))
	{
		ModeManager.ActivateMode(PaintModeID, true);
		FClothingPaintEditMode* PaintMode = (FClothingPaintEditMode*)SkeletalMeshEditor->GetEditorModeManager().GetActiveMode(PaintModeID);
		if (PaintMode)
		{
			PaintMode->SetPersonaToolKit(SkeletalMeshEditor->GetPersonaToolkit());
			PaintMode->SetupClothPaintTab(GetActiveClothTab(InSkeletalMeshEditor));
		}
	}
	else
	{
		ModeManager.DeactivateMode(PaintModeID);
	}
}

TSharedPtr<SClothPaintTab> FClothPaintingModule::GetActiveClothTab(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor, bool bInvoke /*= true*/) const
{
	TSharedPtr<FTabManager> TabManager = InSkeletalMeshEditor.Pin()->GetTabManager();

	if(bInvoke)
	{
		TabManager->TryInvokeTab(FTabId(FClothPaintTabSummoner::TabName));
	}

	// If we can't summon this tab we will have spawned a placeholder which we can not cast as done below,
	// there is no valid active clothing tab so return nullptr
	if(TabManager->HasTabSpawner(FClothPaintTabSummoner::TabName))
	{
		TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(FTabId(FClothPaintTabSummoner::TabName));

		if(Tab.IsValid())
		{
			TSharedPtr<SWidget> TabContent = Tab->GetContent();
			TSharedPtr<SClothPaintTab> ClothingTab = StaticCastSharedPtr<SClothPaintTab>(TabContent);
			return ClothingTab;
		}
	}

	return nullptr;
}

void FClothPaintingModule::ShutdownModule()
{
	ShutdownMode();

	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	// Remove skel mesh editor extenders
	ISkeletalMeshEditorModule* SkelMeshEditorModule = FModuleManager::GetModulePtr<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	if (SkelMeshEditorModule)
	{
		TArray<ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender>& Extenders = SkelMeshEditorModule->GetAllSkeletalMeshEditorToolbarExtenders();

		Extenders.RemoveAll([this](const ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender& InDelegate) {return InDelegate.GetHandle() == SkelMeshEditorExtenderHandle; });
	}
}

void FClothPaintingModule::ShutdownMode()
{
	// Remove extender delegate
	FWorkflowCentricApplication::GetModeExtenderList().RemoveAll([this](FWorkflowApplicationModeExtender& StoredExtender)
	{
		return StoredExtender.GetHandle() == Extender.GetHandle();
	});

	// During shutdown clean up all factories from any modes which are still active/alive
	for(TWeakPtr<FApplicationMode> WeakMode : RegisteredApplicationModes)
	{
		if(WeakMode.IsValid())
		{
			TSharedPtr<FApplicationMode> Mode = WeakMode.Pin();
			Mode->RemoveTabFactory(FClothPaintTabSummoner::TabName);
		}
	}

	RegisteredApplicationModes.Empty();

	FEditorModeRegistry::Get().UnregisterMode(PaintModeID);
}

#undef LOCTEXT_NAMESPACE // "AnimationModifiersModule"
