// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAssetExporter.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ClothingAssetBase.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserModule.h"
#include "Engine/SkinnedAsset.h"
#include "Features/IModularFeatures.h"
#include "FileHelpers.h"
#include "HAL/PlatformMisc.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/TopLevelAssetPath.h"
#include "SkinnedAssetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingAssetExporter)

#define LOCTEXT_NAMESPACE "ClothingAssetExporter"

void ForEachClothingAssetExporter(TFunctionRef<void(UClass*)> Function)
{
	// Add clothing asset exporters
	const TArray<IClothingAssetExporterClassProvider*> ClassProviders = IModularFeatures::Get().GetModularFeatureImplementations<IClothingAssetExporterClassProvider>(IClothingAssetExporterClassProvider::FeatureName);
	for (const IClothingAssetExporterClassProvider* const ClassProvider : ClassProviders)
	{
		if (!ClassProvider)
		{
			continue;
		}
		if (const TSubclassOf<UClothingAssetExporter> ClothingAssetExporterClass = ClassProvider->GetClothingAssetExporterClass())
		{
			UClothingAssetExporter* const ClothingAssetExporter = ClothingAssetExporterClass->GetDefaultObject<UClothingAssetExporter>();
			if (ClothingAssetExporter)
			{
				UClass* const ExportedType = ClothingAssetExporter->GetExportedType();

				Function(ExportedType);
			}
		}
	}
}

void ExportClothingAsset(const UClothingAssetBase* ClothingAsset, UClass* ExportedType)
{
	if (!ClothingAsset || !ExportedType)
	{
		return;
	}

	// Get destination for asset
	FSaveAssetDialogConfig ExportClothingAssetDialogConfig;
	{
		const FString PackageName = ClothingAsset->GetOutermost()->GetName();
		ExportClothingAssetDialogConfig.DefaultPath = FPackageName::GetLongPackagePath(PackageName);
		ExportClothingAssetDialogConfig.DefaultAssetName = ClothingAsset->GetName();
		ExportClothingAssetDialogConfig.AssetClassNames.Add(ExportedType->GetClassPathName());
		ExportClothingAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		ExportClothingAssetDialogConfig.DialogTitleOverride = LOCTEXT("ExportClothingAssetDialogTitle", "Export Clothing Asset As");
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FString NewPackageName;
	FText OutError;
	for (bool bFilenameValid = false; !bFilenameValid; bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError))
	{
		const FString ExportClothingAssetPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(ExportClothingAssetDialogConfig);
		if (ExportClothingAssetPath.IsEmpty())
		{
			return;
		}
		NewPackageName = FPackageName::ObjectPathToPackageName(ExportClothingAssetPath);
	}

	// Try loading the package, when overwriting an existing asset it needs to be loaded first
	if (UPackage* const Package = LoadPackage(nullptr, *NewPackageName, LOAD_Quiet | LOAD_EditorOnly))
	{
		if (USkinnedAsset* const SkinnedAsset = Cast<USkinnedAsset>(Package->FindAssetInPackage()))
		{
			// Finish compilation now otherwise the asset might be deleted and replaced before it ends
			FSkinnedAssetCompilingManager::Get().FinishCompilation({ SkinnedAsset });
		}
	}

	// Find a matching exporter
	UClothingAssetExporter* ClothingAssetExporter = nullptr;

	const TArray<IClothingAssetExporterClassProvider*> ClassProviders = IModularFeatures::Get().GetModularFeatureImplementations<IClothingAssetExporterClassProvider>(IClothingAssetExporterClassProvider::FeatureName);
	for (IClothingAssetExporterClassProvider* const ClassProvider : ClassProviders)
	{
		if (!ClassProvider)
		{
			continue;
		}
		if (const TSubclassOf<UClothingAssetExporter> ClothingAssetExporterClass = ClassProvider->GetClothingAssetExporterClass())
		{
			UClothingAssetExporter* const ClothingAssetExporterCandidate = ClothingAssetExporterClass->GetDefaultObject<UClothingAssetExporter>();

			if (ClothingAssetExporterCandidate && ExportedType == ClothingAssetExporterCandidate->GetExportedType())
			{
				ClothingAssetExporter = ClothingAssetExporterCandidate;
				break;
			}
		}
	}

	if (!ClothingAssetExporter)
	{
		const FText TitleMessage = LOCTEXT("ClothingAssetExporterTitleMessage", "Error Exporting Clothing Asset");
		const FText ErrorMessage = FText::Format(LOCTEXT("ClothingAssetExporterErrorMessage", "The Asset's exporter for type {0} cannot be found anymore. Check that the plugin hasn't been unloaded."), FText::FromName(ExportedType->GetFName()));
		FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, ErrorMessage, TitleMessage);
		return;
	}

	// Create the new asset
	const FName NewAssetName(FPackageName::GetLongPackageAssetName(NewPackageName));
	UPackage* const NewPackage = CreatePackage(*NewPackageName);

	UObject* const NewAsset = NewObject<UObject>(NewPackage, ExportedType, NewAssetName, RF_Public | RF_Standalone | RF_Transactional);

	NewAsset->MarkPackageDirty();

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(NewAsset);

	// Call the export function
	ClothingAssetExporter->Export(ClothingAsset, NewAsset);

	// Save the package
	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(NewAsset->GetOutermost());
	constexpr bool bCheckDirty = false;
	constexpr bool bPromptToSave = false;
	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
}

#undef LOCTEXT_NAMESPACE
