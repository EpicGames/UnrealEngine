// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAssetListCommands.h"

#include "ClothingAssetExporter.h"
#include "Containers/Array.h"
#include "Features/IModularFeatures.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Internationalization/Text.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "ClothingAssetListCommands"

void FClothingAssetListCommands::RegisterCommands()
{
	UI_COMMAND(DeleteAsset, "Delete Asset", "Delete a Clothing Data from the mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteAssetLastLod, "Delete Asset's Last LOD", "Delete the last clothing Data LOD.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RebuildAssetParams, "Rebuild Asset Parameter Masks", "Takes the parameter masks in LOD0 and creates masks in all lower LODs to match, casting those parameter masks to the new mesh.", EUserInterfaceActionType::Button, FInputChord());

	// Add one command per exporter provider
	const TArray<IClothingAssetExporterClassProvider*> ClassProviders = IModularFeatures::Get().GetModularFeatureImplementations<IClothingAssetExporterClassProvider>(IClothingAssetExporterClassProvider::FeatureName);
	const int32 NumClassProviders = ClassProviders.Num();
	ExportAssets.Reserve(NumClassProviders);

	for (const IClothingAssetExporterClassProvider* const ClassProvider : ClassProviders)
	{
		if (ClassProvider)
		{
			if (const TSubclassOf<UClothingAssetExporter> ClothingAssetExporterClass = ClassProvider->GetClothingAssetExporterClass())
			{
				UClothingAssetExporter* const ClothingAssetExporter = ClothingAssetExporterClass->GetDefaultObject<UClothingAssetExporter>();
				if (ClothingAssetExporter)
				{
					UClass* ExportedType = ClothingAssetExporter->GetExportedType();
					const FName CommandName = ExportedType->GetFName();
					const FText ShortName = FText::Format(LOCTEXT("ExportToClothAssetType", "Export To {0}"), FText::FromName(CommandName));
					const FText Description = FText::Format(LOCTEXT("ExportToClothAssetTypeDescription", "Export the selected Clothing Data to the {0} asset type."), FText::FromName(CommandName));

					TSharedPtr<FUICommandInfo>& CommandId = ExportAssets.Add(CommandName);

					FUICommandInfo::MakeCommandInfo(
						this->AsShared(),
						CommandId,
						CommandName,
						ShortName,
						Description,
						FSlateIcon(),
						EUserInterfaceActionType::Button,
						FInputChord());
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
