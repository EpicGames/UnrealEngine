// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FClothingAssetListCommands : public TCommands<FClothingAssetListCommands>
{
public:
	FClothingAssetListCommands()
		: TCommands<FClothingAssetListCommands>(
			TEXT("ClothAssetList"), 
			NSLOCTEXT("Contexts", "ClothAssetList", "Clothing Asset List"), 
			NAME_None, 
			FAppStyle::GetAppStyleSetName())
	{}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> DeleteAsset;
	TSharedPtr<FUICommandInfo> DeleteAssetLastLod;
	TSharedPtr<FUICommandInfo> RebuildAssetParams;
	TMap<FName, TSharedPtr<FUICommandInfo>> ExportAssets;
};
