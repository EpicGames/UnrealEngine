// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingPaintEditMode.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "AssetEditorModeManager.h"
#include "AssetViewerSettings.h"
#include "ClothPainter.h"
#include "ClothingAsset.h"
#include "ClothingAssetBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Array.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "EditorViewportClient.h"
#include "Engine/SkeletalMesh.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "SClothPaintTab.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/Guid.h"
#include "ShowFlags.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

FClothingPaintEditMode::FClothingPaintEditMode()
{
	
}

FClothingPaintEditMode::~FClothingPaintEditMode()
{
	if(ClothPainter.IsValid())
	{
		// Drop the reference
		ClothPainter = nullptr;
	}
}

class IPersonaPreviewScene* FClothingPaintEditMode::GetAnimPreviewScene() const
{
	return static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FClothingPaintEditMode::Initialize()
{
	ClothPainter = MakeShared<FClothPainter>();
	MeshPainter = ClothPainter.Get();

	ClothPainter->Init();
}

TSharedPtr<class FModeToolkit> FClothingPaintEditMode::GetToolkit()
{
	return nullptr;
}

void FClothingPaintEditMode::SetPersonaToolKit(class TSharedPtr<IPersonaToolkit> InToolkit)
{
	PersonaToolkit = InToolkit;
}

void FClothingPaintEditMode::SetupClothPaintTab(TSharedPtr<SClothPaintTab> InClothPaintTab)
{
	ClothPaintTab = InClothPaintTab;
	ClothPaintTab.Pin().Get()->EnterPaintMode();
}

void FClothingPaintEditMode::Enter()
{
	IMeshPaintEdMode::Enter();

	for (FEditorViewportClient* ViewportClient : GEditor->GetAllViewportClients())
	{
		if (!ViewportClient || ViewportClient->GetModeTools() != GetModeManager() )
		{
			continue;
		}

		ViewportClient->EngineShowFlags.DisableAdvancedFeatures();
	}

	IPersonaPreviewScene* Scene = GetAnimPreviewScene();
	if (Scene)
	{
		ClothPainter->SetSkeletalMeshComponent(Scene->GetPreviewMeshComponent());
	}
	
	ClothPainter->EnterPaintMode();
}

void FClothingPaintEditMode::Exit()
{
	IPersonaPreviewScene* Scene = GetAnimPreviewScene();
	if (Scene)
	{
		UDebugSkelMeshComponent* MeshComponent = Scene->GetPreviewMeshComponent();

		if(MeshComponent)
		{
			MeshComponent->bDisableClothSimulation = false;

			if(USkeletalMesh* SkelMesh = MeshComponent->GetSkeletalMeshAsset())
			{
				for(UClothingAssetBase* AssetBase : SkelMesh->GetMeshClothingAssets())
				{
					if (UClothingAssetCommon* const ConcreteAsset = Cast<UClothingAssetCommon>(AssetBase))
					{
						constexpr bool bUpdateFixedVertData = true;
						constexpr bool bInvalidateDerivedDataCache = true;
						ConcreteAsset->ApplyParameterMasks(bUpdateFixedVertData, bInvalidateDerivedDataCache);
					}
				}
			}

			MeshComponent->ResetMeshSectionVisibility();
			MeshComponent->SelectedClothingGuidForPainting = FGuid();
			MeshComponent->SelectedClothingLodForPainting = INDEX_NONE;
			MeshComponent->SelectedClothingLodMaskForPainting = INDEX_NONE;
		}
	}

	if(PersonaToolkit.IsValid())
	{
		if(USkeletalMesh* SkelMesh = PersonaToolkit.Pin()->GetPreviewMesh())
		{
			for(TObjectIterator<USkeletalMeshComponent> It; It; ++It)
			{
				USkeletalMeshComponent* Component = *It;
				if(Component && !Component->IsTemplate() && Component->GetSkeletalMeshAsset() == SkelMesh)
				{
					Component->ReregisterComponent();
				}
			}
		}
	}

	for (FEditorViewportClient* ViewportClient : GEditor->GetAllViewportClients())
	{
		if (!ViewportClient || ViewportClient->GetModeTools() != GetModeManager() )
		{
			continue;
		}

		const bool bEnablePostProcessing = UAssetViewerSettings::Get()->Profiles[GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex].bPostProcessingEnabled;

		if ( bEnablePostProcessing )
		{
			ViewportClient->EngineShowFlags.EnableAdvancedFeatures();
		}
		else
		{
			ViewportClient->EngineShowFlags.DisableAdvancedFeatures();
		}
	}

	ClothPainter->ExitPaintMode();

	if (ClothPaintTab.IsValid())
	{
		ClothPaintTab.Pin().Get()->ExitPaintMode();	
	}

	IMeshPaintEdMode::Exit();
}
