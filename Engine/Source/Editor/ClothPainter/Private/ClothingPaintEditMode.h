// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMeshPaintMode.h"
#include "Templates/SharedPointer.h"

class SClothPaintTab;
class FClothPainter;
class IPersonaToolkit;

class FClothingPaintEditMode : public IMeshPaintEdMode
{
public:
	FClothingPaintEditMode();
	virtual ~FClothingPaintEditMode();

	virtual void Initialize() override;
	virtual bool UsesToolkits() const override { return false; }
	virtual TSharedPtr<class FModeToolkit> GetToolkit() override;

	void SetPersonaToolKit(class TSharedPtr<IPersonaToolkit> InToolkit);
	void SetupClothPaintTab(TSharedPtr<SClothPaintTab> InClothPaintTab);

	virtual void Enter() override;
	virtual void Exit() override;

protected:

	TSharedPtr<FClothPainter> ClothPainter;

	class IPersonaPreviewScene* GetAnimPreviewScene() const;
	class TWeakPtr<IPersonaToolkit> PersonaToolkit;
	TWeakPtr<SClothPaintTab> ClothPaintTab;
};