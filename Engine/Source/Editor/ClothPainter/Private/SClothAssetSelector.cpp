// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClothAssetSelector.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "ClothLODData.h"
#include "ClothPhysicalMeshData.h"
#include "ClothingAsset.h"
#include "ClothingAssetBase.h"
#include "ClothingAssetExporter.h"
#include "ClothingAssetFactoryInterface.h"
#include "ClothingAssetListCommands.h"
#include "ClothingSimulationFactory.h"
#include "ClothingSystemEditorInterfaceModule.h"
#include "Containers/ContainersFwd.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/SkeletalMesh.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "IContentBrowserSingleton.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PointWeightMap.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SCopyVertexColorSettingsPanel.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "SkeletalMeshTypes.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Utils/ClothingMeshUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"

class FUICommandInfo;
class ITableRow;
class STableViewBase;
class SWidget;
class UObject;
struct FGeometry;

#define LOCTEXT_NAMESPACE "ClothAssetSelector"

FPointWeightMap* FClothingMaskListItem::GetMask()
{
	if(UClothingAssetCommon* Asset = ClothingAsset.Get())
	{
		if(Asset->IsValidLod(LodIndex))
		{
			FClothLODDataCommon& LodData = Asset->LodData[LodIndex];
			if(LodData.PointWeightMaps.IsValidIndex(MaskIndex))
			{
				return &LodData.PointWeightMaps[MaskIndex];
			}
		}
	}

	return nullptr;
}

FClothPhysicalMeshData* FClothingMaskListItem::GetMeshData()
{
	UClothingAssetCommon* Asset = ClothingAsset.Get();
	return Asset && Asset->IsValidLod(LodIndex) ? &Asset->LodData[LodIndex].PhysicalMeshData : nullptr;
}

USkeletalMesh* FClothingMaskListItem::GetOwningMesh()
{
	UClothingAssetCommon* Asset = ClothingAsset.Get();
	return Asset ? Cast<USkeletalMesh>(Asset->GetOuter()) : nullptr;
}

class SAssetListRow : public STableRow<TSharedPtr<FClothingAssetListItem>>
{
public:

	SLATE_BEGIN_ARGS(SAssetListRow)
	{}
		SLATE_EVENT(FSimpleDelegate, OnInvalidateList)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FClothingAssetListItem> InItem)
	{
		Item = InItem;
		OnInvalidateList = InArgs._OnInvalidateList;

		BindCommands();

		STableRow<TSharedPtr<FClothingAssetListItem>>::Construct(
			STableRow<TSharedPtr<FClothingAssetListItem>>::FArguments()
			.Content()
			[
				SNew(SBox)
				.Padding(2.0f)
				[
					SAssignNew(EditableText, SInlineEditableTextBlock)
					.Text(this, &SAssetListRow::GetAssetName)
					.OnTextCommitted(this, &SAssetListRow::OnCommitAssetName)
					.IsSelected(this, &SAssetListRow::IsSelected)
				]
			],
			InOwnerTable
		);
	}

	FText GetAssetName() const
	{
		if(Item.IsValid())
		{
			return FText::FromString(Item->ClothingAsset->GetName());
		}

		return FText::GetEmpty();
	}

	void OnCommitAssetName(const FText& InText, ETextCommit::Type CommitInfo)
	{
		if(Item.IsValid())
		{
			if(UClothingAssetBase* const Asset = Item->ClothingAsset.Get())
			{
				if (const FText TrimText = FText::TrimPrecedingAndTrailing(InText); !TrimText.IsEmpty())
				{
					const FString SanitizedName = ObjectTools::SanitizeObjectName(TrimText.ToString());

					if(Asset->GetName() != SanitizedName)
					{
						// Check for an existing object, and if we find one build a unique name based on the request
						const FString NewName = StaticFindObject(UClothingAssetBase::StaticClass(), Asset->GetOuter(), *SanitizedName) ?
							MakeUniqueObjectName(Asset->GetOuter(), UClothingAssetBase::StaticClass(), FName(SanitizedName)).ToString() :
							SanitizedName;

						FScopedTransaction CurrTransaction(LOCTEXT("RenameClothingAsset_Transaction", "Rename Clothing Data."));
						Asset->Modify();
						Asset->Rename(*NewName, Asset->GetOuter());
					}
				}
			}
		}
	}

	void BindCommands()
	{
		check(!UICommandList.IsValid());

		UICommandList = MakeShareable(new FUICommandList);

		const FClothingAssetListCommands& Commands = FClothingAssetListCommands::Get();

		UICommandList->MapAction(
			FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SAssetListRow::RenameAsset)
		);

		UICommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SAssetListRow::DeleteAsset)
		);

		UICommandList->MapAction(
			Commands.DeleteAssetLastLod,
			FExecuteAction::CreateSP(this, &SAssetListRow::DeleteAssetLastLod),
			FCanExecuteAction::CreateSP(this, &SAssetListRow::CanDeleteAssetLastLod)
		);

		UICommandList->MapAction(
			Commands.RebuildAssetParams,
			FExecuteAction::CreateSP(this, &SAssetListRow::RebuildLODParameters),
			FCanExecuteAction::CreateSP(this, &SAssetListRow::CanRebuildLODParameters)
		);

		// Add clothing asset exporters
		ForEachClothingAssetExporter([this, &Commands](UClass* ExportedType)
			{
				if (const TSharedPtr<FUICommandInfo>* const CommandId = Commands.ExportAssets.Find(ExportedType->GetFName()))
				{
					UICommandList->MapAction(
						*CommandId,
						FExecuteAction::CreateSP(this, &SAssetListRow::ExportAsset, ExportedType));
				}
			});
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if(Item.IsValid() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			const UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(Item->ClothingAsset.Get());  // Only the UClothingAssetCommon type have LODs
			const int32 LastLodIndex = Asset ? Asset->GetNumLods() - 1 : INDEX_NONE;

			const FClothingAssetListCommands& Commands = FClothingAssetListCommands::Get();
			FMenuBuilder Builder(true, UICommandList);

			Builder.BeginSection(NAME_None, LOCTEXT("AssetActions_SectionName", "Actions"));
			{
				Builder.AddMenuEntry(FGenericCommands::Get().Rename);
				Builder.AddSeparator();
				Builder.AddMenuEntry(FGenericCommands::Get().Delete);
				if (LastLodIndex > 0)  // Can't delete LOD0
				{
					Builder.AddMenuEntry(Commands.DeleteAssetLastLod, NAME_None,
						FText::Format(LOCTEXT("DeleteAssetLastLod", "Delete LOD{0}"), LastLodIndex));
				}
				Builder.AddMenuEntry(Commands.RebuildAssetParams);

				for (const TPair<FName, TSharedPtr<FUICommandInfo>>& CommandId : Commands.ExportAssets)
				{
					Builder.AddMenuEntry(CommandId.Value);
				}
			}
			Builder.EndSection();

			FWidgetPath Path = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

			FSlateApplication::Get().PushMenu(AsShared(), Path, Builder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu);

			return FReply::Handled();
		}

		return STableRow<TSharedPtr<FClothingAssetListItem>>::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

private:
	
	void RenameAsset()
	{
		if (EditableText)
		{
			EditableText->EnterEditingMode();
		}
	}


	void DeleteAsset()
	{
		//Lambda use to sync one of the UserSectionData section from one LOD Model
		auto SetSkelMeshSourceSectionUserData = [](FSkeletalMeshLODModel& LODModel, const int32 SectionIndex, const int32 OriginalSectionIndex)
		{
			FSkelMeshSourceSectionUserData& SourceSectionUserData = LODModel.UserSectionsData.FindOrAdd(OriginalSectionIndex);
			SourceSectionUserData.bDisabled = LODModel.Sections[SectionIndex].bDisabled;
			SourceSectionUserData.bCastShadow = LODModel.Sections[SectionIndex].bCastShadow;
			SourceSectionUserData.bVisibleInRayTracing = LODModel.Sections[SectionIndex].bVisibleInRayTracing;			
			SourceSectionUserData.bRecomputeTangent = LODModel.Sections[SectionIndex].bRecomputeTangent;
			SourceSectionUserData.GenerateUpToLodIndex = LODModel.Sections[SectionIndex].GenerateUpToLodIndex;
			SourceSectionUserData.CorrespondClothAssetIndex = LODModel.Sections[SectionIndex].CorrespondClothAssetIndex;
			SourceSectionUserData.ClothingData = LODModel.Sections[SectionIndex].ClothingData;
		};

		if(UClothingAssetBase* Asset = Item->ClothingAsset.Get())
		{
			if(USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(Asset->GetOuter()))
			{
				FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
				int32 AssetIndex;
				if(SkelMesh->GetMeshClothingAssets().Find(Asset, AssetIndex))
				{
					FScopedTransaction CurrTransaction(LOCTEXT("DeleteClothingAsset_Transaction", "Delete Clothing Data."));
					SkelMesh->Modify();
					Asset->Modify();

					// Need to unregister our components so they shut down their current clothing simulation
					FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
					SkelMesh->PreEditChange(nullptr);

					Asset->UnbindFromSkeletalMesh(SkelMesh, INDEX_NONE, INDEX_NONE);
					SkelMesh->GetMeshClothingAssets().RemoveAt(AssetIndex);
					Asset->Rename(nullptr, GetTransientPackage());  // Rename so that the old object name can be reused if needed
					Asset = nullptr;

					// Need to fix up asset indices on sections.
					if(FSkeletalMeshModel* Model = SkelMesh->GetImportedModel())
					{
						for(FSkeletalMeshLODModel& LodModel : Model->LODModels)
						{
							for (int32 SectionIndex = 0; SectionIndex < LodModel.Sections.Num(); ++SectionIndex)
							{
								FSkelMeshSection& Section = LodModel.Sections[SectionIndex];
								if(Section.CorrespondClothAssetIndex > AssetIndex)
								{
									--Section.CorrespondClothAssetIndex;
									//Keep the user section data (build source data) in sync
									SetSkelMeshSourceSectionUserData(LodModel, SectionIndex, Section.OriginalDataSectionIndex);
								}
							}
						}
					}
					OnInvalidateList.ExecuteIfBound();
				}
			}
		}
	}

	bool CanDeleteAssetLastLod()
	{
		if (Item.IsValid())
		{
			if (UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(Item->ClothingAsset.Get()))  // Only the UClothingAssetCommon type have LODs
			{
				return Asset->GetNumLods() > 1;  // Can't delete LOD0
			}
		}
		return false;
	}

	void DeleteAssetLastLod()
	{
		if (Item.IsValid())
		{
			if (UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(Item->ClothingAsset.Get()))  // Only the UClothingAssetCommon type have LODs
			{
				if (ensure(Asset->GetNumLods() > 1))  // Can't delete LOD0
				{
					if (USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(Asset->GetOuter()))
					{
						FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkeletalMesh);
						FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
						FScopedTransaction CurrTransaction(LOCTEXT("DeleteLastLod_Transaction", "Delete the last Clothing Data LOD."));
						SkeletalMesh->Modify();
						Asset->Modify();
						SkeletalMesh->PreEditChange(nullptr);
						Asset->RemoveLastLod();
						OnInvalidateList.ExecuteIfBound();
					}
				}
			}
		}
	}

	// Using LOD0 of an asset, rebuild the other LOD masks by mapping the LOD0 parameters onto their meshes
	void RebuildLODParameters()
	{
		if(!Item.IsValid())
		{
			return;
		}

		if(UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(Item->ClothingAsset.Get()))
		{
			const int32 NumLods = Asset->GetNumLods();

			for(int32 CurrIndex = 0; CurrIndex < NumLods - 1; ++CurrIndex)
			{
				FClothLODDataCommon& SourceLod = Asset->LodData[CurrIndex];
				FClothLODDataCommon& DestLod = Asset->LodData[CurrIndex + 1];

				DestLod.PointWeightMaps.Reset();

				for(FPointWeightMap& SourceMask : SourceLod.PointWeightMaps)
				{
					DestLod.PointWeightMaps.AddDefaulted();
					FPointWeightMap& DestMask = DestLod.PointWeightMaps.Last();

					DestMask.Name = SourceMask.Name;
					DestMask.bEnabled = SourceMask.bEnabled;
					DestMask.CurrentTarget = SourceMask.CurrentTarget;

					ClothingMeshUtils::FVertexParameterMapper ParameterMapper(
						DestLod.PhysicalMeshData.Vertices,
						DestLod.PhysicalMeshData.Normals,
						SourceLod.PhysicalMeshData.Vertices,
						SourceLod.PhysicalMeshData.Normals,
						SourceLod.PhysicalMeshData.Indices
					);

					ParameterMapper.Map(SourceMask.Values, DestMask.Values);
				}
			}
		}
	}

	void ExportAsset(UClass* ExportedType)
	{
		if (!Item.IsValid() || !ExportedType)
		{
			return;
		}

		if (const UClothingAssetBase* const ClothingAsset = Item->ClothingAsset.Get())
		{
			ExportClothingAsset(ClothingAsset, ExportedType);
		}
	}

	bool CanRebuildLODParameters() const
	{
		if(!Item.IsValid())
		{
			return false;
		}

		if(const UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(Item->ClothingAsset.Get()))
		{
			if(Asset->GetNumLods() > 1)
			{
				return true;
			}
		}

		return false;
	}

	TSharedPtr<FClothingAssetListItem> Item;
	TSharedPtr<SInlineEditableTextBlock> EditableText;
	FSimpleDelegate OnInvalidateList;
	TSharedPtr<FUICommandList> UICommandList;
};

class SMaskListRow : public SMultiColumnTableRow<TSharedPtr<FClothingMaskListItem>>
{
public:

	SLATE_BEGIN_ARGS(SMaskListRow)
	{}

		SLATE_EVENT(FSimpleDelegate, OnInvalidateList)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FClothingMaskListItem> InItem, TSharedPtr<SClothAssetSelector> InAssetSelector )
	{
		OnInvalidateList = InArgs._OnInvalidateList;
		Item = InItem;
		AssetSelectorPtr = InAssetSelector;

		BindCommands();

		SMultiColumnTableRow<TSharedPtr<FClothingMaskListItem>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	static FName Column_Enabled;
	static FName Column_MaskName;
	static FName Column_CurrentTarget;

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if(InColumnName == Column_Enabled)
		{
			return SNew(SBox)
			.Padding(2.f)
			[
				SNew(SCheckBox)
				.IsEnabled(this, &SMaskListRow::IsMaskCheckboxEnabled, Item)
				.IsChecked(this, &SMaskListRow::IsMaskEnabledChecked, Item)
				.OnCheckStateChanged(this, &SMaskListRow::OnMaskEnabledCheckboxChanged, Item)
				.ToolTipText(LOCTEXT("MaskEnableCheckBox_ToolTip", "Sets whether this mask is enabled and can affect final parameters for its target parameter."))
			];
		}

		if(InColumnName == Column_MaskName)
		{
			return SAssignNew(InlineText, SInlineEditableTextBlock)
				.Text(this, &SMaskListRow::GetMaskName)
				.OnTextCommitted(this, &SMaskListRow::OnCommitMaskName)
				.IsSelected(this, &SMultiColumnTableRow<TSharedPtr<FClothingMaskListItem>>::IsSelectedExclusively);
		}

		if(InColumnName == Column_CurrentTarget)
		{
			// Retrieve the mask names for the current clothing simulation factory
			if (const TSharedPtr<const SClothAssetSelector> AssetSelector = AssetSelectorPtr.Pin())
			{
				if (const TStrongObjectPtr<const UClothingAssetBase> ClothingAsset = AssetSelector->GetSelectedAsset().Pin())
				{
					if (const UClothingSimulationFactory* const ClothingSimulationFactory = UClothingSimulationFactory::GetClothingSimulationFactory(ClothingAsset.Get()))
					{
						const UEnum* const Enum = ClothingSimulationFactory->GetWeightMapTargetEnum();
						const FPointWeightMap* const Mask = Item->GetMask();
						if (Enum && Mask)
						{
							return SNew(STextBlock).Text(Enum->GetDisplayNameTextByValue((int64)Mask->CurrentTarget));
						}
					}
				}
			}
		}

		return SNullWidget::NullWidget;
	}

	FText GetMaskName() const
	{
		if(Item.IsValid())
		{
			if(FPointWeightMap* Mask = Item->GetMask())
			{
				return FText::FromName(Mask->Name);
			}
		}

		return LOCTEXT("MaskName_Invalid", "Invalid Mask");
	}

	void OnCommitMaskName(const FText& InText, ETextCommit::Type CommitInfo)
	{
		if(Item.IsValid())
		{
			if(FPointWeightMap* Mask = Item->GetMask())
			{
				FText TrimText = FText::TrimPrecedingAndTrailing(InText);
				Mask->Name = FName(*TrimText.ToString());
			}
		}
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// Spawn menu
		if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && Item.IsValid())
		{
			if(FPointWeightMap* Mask = Item->GetMask())
			{
				FMenuBuilder Builder(true, UICommandList);

				FUIAction DeleteAction(FExecuteAction::CreateSP(this, &SMaskListRow::OnDeleteMask));

				Builder.BeginSection(NAME_None, LOCTEXT("MaskActions_SectionName", "Actions"));
				{
					Builder.AddSubMenu(LOCTEXT("MaskActions_SetTarget", "Set Target"), LOCTEXT("MaskActions_SetTarget_Tooltip", "Choose the target for this mask"), FNewMenuDelegate::CreateSP(this, &SMaskListRow::BuildTargetSubmenu));
					Builder.AddMenuEntry(FGenericCommands::Get().Delete);
					Builder.AddSubMenu(LOCTEXT("MaskActions_CopyFromMask", "Copy From Mask"), LOCTEXT("MaskActions_CopyFromMask_Tooltip", "Replace this mask with values from another mask on this cloth"), FNewMenuDelegate::CreateSP(this, &SMaskListRow::BuildCopyMaskSubmenu));
					Builder.AddSubMenu(LOCTEXT("MaskActions_CopyFromVertexColor", "Copy From Vertex Color"), LOCTEXT("MaskActions_CopyFromVertexColor_Tooltip", "Replace this mask with values from vertex color channel on sim mesh"), FNewMenuDelegate::CreateSP(this, &SMaskListRow::BuildCopyVertexColorSubmenu));
				}
				Builder.EndSection();

				FWidgetPath Path = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

				FSlateApplication::Get().PushMenu(AsShared(), Path, Builder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu);

				return FReply::Handled();
			}
		}

		return SMultiColumnTableRow<TSharedPtr<FClothingMaskListItem>>::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	void EditName()
	{
		if(InlineText.IsValid())
		{
			InlineText->EnterEditingMode();
		}
	}

private:

	void BindCommands()
	{
		check(!UICommandList.IsValid());

		UICommandList = MakeShareable(new FUICommandList);

		UICommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SMaskListRow::OnDeleteMask)
		);
	}

	FClothLODDataCommon* GetCurrentLod() const
	{
		if(Item.IsValid())
		{
			if(UClothingAssetCommon* Asset = Item->ClothingAsset.Get())
			{
				if(Asset->LodData.IsValidIndex(Item->LodIndex))
				{
					return &Asset->LodData[Item->LodIndex];
				}
			}
		}

		return nullptr;
	}

	void OnDeleteMask()
	{
		USkeletalMesh* CurrentMesh = Item->GetOwningMesh();

		if(CurrentMesh)
		{
			FScopedTransaction CurrTransaction(LOCTEXT("DeleteMask_Transaction", "Delete clothing parameter mask."));
			Item->ClothingAsset->Modify();

		if(FClothLODDataCommon* LodData = GetCurrentLod())
		{
			if(LodData->PointWeightMaps.IsValidIndex(Item->MaskIndex))
			{
				LodData->PointWeightMaps.RemoveAt(Item->MaskIndex);

				// We've removed a mask, so it will need to be applied to the clothing data
				if(Item.IsValid())
				{
					if(UClothingAssetCommon* Asset = Item->ClothingAsset.Get())
					{
						Asset->ApplyParameterMasks();
					}
				}

				OnInvalidateList.ExecuteIfBound();
			}
		}
	}
	}

	void OnSetTarget(int32 InTargetEntryIndex)
	{
		USkeletalMesh* CurrentMesh = Item->GetOwningMesh();

		if(Item.IsValid() && CurrentMesh)
		{
			FScopedTransaction CurrTransaction(LOCTEXT("SetMaskTarget_Transaction", "Set clothing parameter mask target."));
			Item->ClothingAsset->Modify();

			if(FPointWeightMap* Mask = Item->GetMask())
			{
				Mask->CurrentTarget = (uint8)InTargetEntryIndex;
				if(Mask->CurrentTarget == (uint8)EWeightMapTargetCommon::None)
				{
					// Make sure to disable this mask if it has no valid target
					Mask->bEnabled = false;
				}

				OnInvalidateList.ExecuteIfBound();
			}
		}
	}

	void OnCopyMask(int32 InMaskIndex)
	{
		if (AssetSelectorPtr.IsValid() && Item.IsValid())
		{
			if (USkeletalMesh* CurrentMesh = Item->GetOwningMesh())
			{
				if (FPointWeightMap* Mask = Item->GetMask())
				{
					if (UClothingAssetCommon* const ClothingAsset = Cast<UClothingAssetCommon>(AssetSelectorPtr.Pin()->GetSelectedAsset().Get()))
					{
						const int32 LOD = AssetSelectorPtr.Pin()->GetSelectedLod();
						if (ClothingAsset->LodData.IsValidIndex(LOD))
						{
							const FClothLODDataCommon& LodData = ClothingAsset->LodData[LOD];
							if (LodData.PointWeightMaps.IsValidIndex(InMaskIndex))
							{
								{
									FScopedTransaction CurrTransaction(LOCTEXT("CopyMask_Transaction", "Copy Mask."));
									Item->ClothingAsset->Modify();
									Mask->Values = LodData.PointWeightMaps[InMaskIndex].Values;
								}

								const bool bUpdateFixedVertData = (Mask->CurrentTarget == (uint8)EWeightMapTargetCommon::MaxDistance);
								ClothingAsset->ApplyParameterMasks(bUpdateFixedVertData);
							}
						}
					}
				}
			}
		}
	}

	void BuildTargetSubmenu(FMenuBuilder& Builder)
	{
		Builder.BeginSection(NAME_None, LOCTEXT("MaskTargets_SectionName", "Targets"));
		{
			// Retrieve the mask names for the current clothing simulation factory
			if (const TSharedPtr<const SClothAssetSelector> AssetSelector = AssetSelectorPtr.Pin())
			{
				if (const TStrongObjectPtr<const UClothingAssetBase> ClothingAsset = AssetSelector->GetSelectedAsset().Pin())
				{
					if (const UClothingSimulationFactory* const ClothingSimulationFactory = UClothingSimulationFactory::GetClothingSimulationFactory(ClothingAsset.Get()))
					{
						if (const UEnum* const Enum = ClothingSimulationFactory->GetWeightMapTargetEnum())
						{
							const int32 NumEntries = Enum->NumEnums();

							// Iterate to -1 to skip the _MAX entry appended to the end of the enum
							for (int32 Index = 0; Index < NumEntries - 1; ++Index)
							{
								FUIAction EntryAction(FExecuteAction::CreateSP(this, &SMaskListRow::OnSetTarget, (int32)Enum->GetValueByIndex(Index)));

								FText EntryText = Enum->GetDisplayNameTextByIndex(Index);

								Builder.AddMenuEntry(EntryText, FText::GetEmpty(), FSlateIcon(), EntryAction);
							}
						}
					}
				}
			}
		}
		Builder.EndSection();
	}

	/** Build sub menu for choosing which vertex color channel to copy to selected mask */
	void BuildCopyMaskSubmenu(FMenuBuilder& Builder)
	{
		if (AssetSelectorPtr.IsValid() && Item.IsValid())
		{
			if (const UClothingAssetCommon* const ClothingAsset = Cast<const UClothingAssetCommon>(AssetSelectorPtr.Pin()->GetSelectedAsset().Get()))
			{
				const int32 LOD = AssetSelectorPtr.Pin()->GetSelectedLod();
				if (ClothingAsset->LodData.IsValidIndex(LOD))
				{
					const FClothLODDataCommon& LodData = ClothingAsset->LodData[LOD];
					Builder.BeginSection(NAME_None, LOCTEXT("Masks_SectionName", "Masks"));
					for (int32 MaskIndex = 0; MaskIndex < LodData.PointWeightMaps.Num(); ++MaskIndex)
					{
						if (MaskIndex == Item->MaskIndex)
						{
							continue;
						}
						FUIAction EntryAction(FExecuteAction::CreateSP(this, &SMaskListRow::OnCopyMask, MaskIndex));

						Builder.AddMenuEntry(FText::FromName(LodData.PointWeightMaps[MaskIndex].Name), FText::GetEmpty(), FSlateIcon(), EntryAction);
						
					}
					Builder.EndSection();
				}
			}
		}
	}

	/** Build sub menu for choosing which vertex color channel to copy to selected mask */
	void BuildCopyVertexColorSubmenu(FMenuBuilder& Builder)
	{
		if (AssetSelectorPtr.IsValid())
		{
			if (UClothingAssetCommon* const ClothingAsset = Cast<UClothingAssetCommon>(AssetSelectorPtr.Pin()->GetSelectedAsset().Get()))
			{
				const int32 LOD = AssetSelectorPtr.Pin()->GetSelectedLod();
				FPointWeightMap* const Mask = Item->GetMask();

				const TSharedRef<SWidget> Widget = SNew(SCopyVertexColorSettingsPanel, ClothingAsset, LOD, Mask);

				Builder.AddWidget(Widget, FText::GetEmpty(), true, false);
			}
		}
	}

	// Mask enabled checkbox handling
	ECheckBoxState IsMaskEnabledChecked(TSharedPtr<FClothingMaskListItem> InItem) const
	{
		if(InItem.IsValid())
		{
			if(FPointWeightMap* Mask = InItem->GetMask())
			{
				return Mask->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}

		return ECheckBoxState::Unchecked;
	}

	bool IsMaskCheckboxEnabled(TSharedPtr<FClothingMaskListItem> InItem) const
	{
		if(InItem.IsValid())
		{
			if(FPointWeightMap* Mask = InItem->GetMask())
			{
				return Mask->CurrentTarget != (uint8)EWeightMapTargetCommon::None;
			}
		}

		return false;
	}

	void OnMaskEnabledCheckboxChanged(ECheckBoxState InState, TSharedPtr<FClothingMaskListItem> InItem)
	{
		if(InItem.IsValid())
		{
			if(FPointWeightMap* Mask = InItem->GetMask())
			{
				const bool bNewEnableState = (InState == ECheckBoxState::Checked);

				if(Mask->bEnabled != bNewEnableState)
				{
					if(bNewEnableState)
					{
						// Disable all other masks that affect this target (there can only be one mask enabled of the same target type at the same time)
						if(UClothingAssetCommon* Asset = InItem->ClothingAsset.Get())
						{
							if(Asset->LodData.IsValidIndex(InItem->LodIndex))
							{
								FClothLODDataCommon& LodData = Asset->LodData[InItem->LodIndex];

								TArray<FPointWeightMap*> AllTargetMasks;
								LodData.GetParameterMasksForTarget(Mask->CurrentTarget, AllTargetMasks);

								for(FPointWeightMap* TargetMask : AllTargetMasks)
								{
									if(TargetMask && TargetMask != Mask)
									{
										TargetMask->bEnabled = false;
									}
								}
							}
						}
					}

					// Set the flag
					Mask->bEnabled = bNewEnableState;
					
					if(UClothingAssetCommon* Asset = InItem->ClothingAsset.Get())
					{
						const bool bUpdateFixedVertData = (Mask->CurrentTarget == (uint8)EWeightMapTargetCommon::MaxDistance);
						Asset->ApplyParameterMasks(bUpdateFixedVertData);
					}
				}
			}
		}
	}

	FSimpleDelegate OnInvalidateList;
	TSharedPtr<FClothingMaskListItem> Item;
	TSharedPtr<SInlineEditableTextBlock> InlineText;
	TSharedPtr<FUICommandList> UICommandList;
	TWeakPtr<SClothAssetSelector> AssetSelectorPtr;
};

FName SMaskListRow::Column_Enabled(TEXT("Enabled"));
FName SMaskListRow::Column_MaskName(TEXT("Name"));
FName SMaskListRow::Column_CurrentTarget(TEXT("CurrentTarget"));

SClothAssetSelector::~SClothAssetSelector()
{
	if(Mesh)
	{
		Mesh->UnregisterOnClothingChange(MeshClothingChangedHandle);
	}

	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SClothAssetSelector::Construct(const FArguments& InArgs, USkeletalMesh* InMesh)
{
	FClothingAssetListCommands::Register();

	Mesh = InMesh;
	OnSelectionChanged = InArgs._OnSelectionChanged;

	// Register callback for external changes to clothing items
	if(Mesh)
	{
		MeshClothingChangedHandle = Mesh->RegisterOnClothingChange(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SClothAssetSelector::OnRefresh));
	}

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.HeaderContent()
			[
				SAssignNew(AssetHeaderBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AssetExpander_Title", "Clothing Data"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SPositiveActionButton)
					.Text(LOCTEXT("CopyClothingFromMeshText_TEXT", "Add Clothing"))
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.ToolTipText(LOCTEXT("CopyClothingFromMeshText_TOOLTIP", "Create clothing data from another asset."))
					.OnGetMenuContent(this, &SClothAssetSelector::OnGenerateSkeletalMeshPickerForClothCopy)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SComboButton)
					.Visibility_Lambda([this]() -> EVisibility { return CanSelectLod() ? EVisibility::Visible : EVisibility::Collapsed; })
					.ForegroundColor(FSlateColor::UseStyle())
					.OnGetMenuContent(this, &SClothAssetSelector::OnGetLodMenu)
					.HasDownArrow(true)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &SClothAssetSelector::GetLodButtonText)
					]
				]
			]
			.BodyContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.f, 2.0f))
				.MinDesiredHeight(100.0f)
				[
					SAssignNew(AssetList, SAssetList)
					.ListItemsSource(&AssetListItems)
					.OnGenerateRow(this, &SClothAssetSelector::OnGenerateWidgetForClothingAssetItem)
					.OnSelectionChanged(this, &SClothAssetSelector::OnAssetListSelectionChanged)
					.ClearSelectionOnClick(false)
					.SelectionMode(ESelectionMode::Single)
				]
			]
		]

		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.Visibility_Lambda([this]() -> EVisibility { return CanAddNewMask() ? EVisibility::Visible : EVisibility::Collapsed; })
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.Padding(FMargin(0.f, 2.0f))
			.HeaderContent()
			[
				SAssignNew(MaskHeaderBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaskExpander_Title", "Masks"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SAssignNew(NewMaskButton, SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked(this, &SClothAssetSelector::AddNewMask)
					.IsEnabled(this, &SClothAssetSelector::CanAddNewMask)
					.ToolTipText(LOCTEXT("AddMask_Tooltip", "Add a Mask"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
					]
				]
			]
			.BodyContent()
			[
				SNew(SBox)
				.MinDesiredHeight(100.0f)
				[
					SAssignNew(MaskList, SMaskList)
					.ListItemsSource(&MaskListItems)
					.OnGenerateRow(this, &SClothAssetSelector::OnGenerateWidgetForMaskItem)
					.OnSelectionChanged(this, &SClothAssetSelector::OnMaskSelectionChanged)
					.ClearSelectionOnClick(false)
					.SelectionMode(ESelectionMode::Single)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(SMaskListRow::Column_Enabled)
						.FixedWidth(40)
						.HAlignCell(HAlign_Right) .DefaultLabel(FText::GetEmpty())

						+ SHeaderRow::Column(SMaskListRow::Column_MaskName)
						.FillWidth(0.5f)
						.DefaultLabel(LOCTEXT("MaskListHeader_Name", "Name"))

						+ SHeaderRow::Column(SMaskListRow::Column_CurrentTarget)
						.FillWidth(0.3f)
						.DefaultLabel(LOCTEXT("MaskListHeader_Target", "Target"))
					)
				]
			]
		]
		+ SVerticalBox::Slot()					// Mesh to mesh skinning
		.AutoHeight()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
		[
			SNew(SExpandableArea)
			.Visibility_Lambda([this]() -> EVisibility { return CanEditDeformer() ? EVisibility::Visible : EVisibility::Collapsed; })
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.Padding(FMargin(0.f, 2.0f))
			.HeaderContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MeshProxyDeformer", "Mesh Proxy Deformer"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
				]
			]
			.BodyContent()
			[	
				// TODO: Replace this with a table view or something more suitable. UETOOL-2341 

				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)
				.PhysicalSplitterHandleSize(1.0)

				+ SSplitter::Slot()
				.Value(.3f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(16.0f, 2.0f, 0.0f, 2.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
						.Text(LOCTEXT("MultipleInfluences", "Use Multiple Influences"))
						.ToolTipText(LOCTEXT("MultipleInfluencesToolTip", "When enabled, use multiple triangles from the simulation mesh to deform the render section, instead of a single one."))
					]

					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(16.0f, 2.0f, 0.0f, 2.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
						.Text(LOCTEXT("SearchRadius", "Search Radius"))
						.ToolTipText(LOCTEXT("SearchRadiusToolTip", "The search radius around the render section vertices to look for all the influencing simulation triangles."))
					]
					
					+ SVerticalBox::Slot()
					.Padding(16.0f, 2.0f, 0.0f, 2.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
						.Text(LOCTEXT("SmoothTransition", "Smooth Transition"))
						.ToolTipText(LOCTEXT("SmoothTransitionToolTip", "Whether to enable smooth transitions between kinematic and dynamic simulation triangles deforming the render section."))
					]
				]

				+ SSplitter::Slot()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(12.0f, 4.0f, 0.0f, 4.0f)
					[
						SNew(SCheckBox)
						.IsChecked(this, &SClothAssetSelector::GetCurrentUseMultipleInfluences)
						.OnCheckStateChanged(this, &SClothAssetSelector::OnCurrentUseMultipleInfluencesChanged)
						.IsEnabled(this, &SClothAssetSelector::IsValidClothLodSelected)
					]

					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(12.0f, 4.0f, 0.0f, 4.0f)
					[
						SNew(SNumericEntryBox<float>)
						.AllowSpin(true)
						.MinSliderValue(0)
						.MinValue(0)
						.MaxSliderValue(TOptional<float>(1000.0f))
						.IsEnabled(this, &SClothAssetSelector::CurrentKernelRadiusIsEnabled)
						.UndeterminedString(FText::FromString("????"))
						.Value(this, &SClothAssetSelector::GetCurrentKernelRadius)
						.OnValueCommitted(this, &SClothAssetSelector::OnCurrentKernelRadiusCommitted)
						.OnValueChanged(this, &SClothAssetSelector::OnCurrentKernelRadiusChanged)
						.LabelPadding(0)
					]
					
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(12.0f, 4.0f, 0.0f, 4.0f)
					[
						SNew(SCheckBox)
						.IsChecked(this, &SClothAssetSelector::GetCurrentSmoothTransition)
						.OnCheckStateChanged(this, &SClothAssetSelector::OnCurrentSmoothTransitionChanged)
						.IsEnabled(this, &SClothAssetSelector::IsValidClothLodSelected)
					]
				]
			]
		]
	];

	RefreshAssetList();
	RefreshMaskList();
}

TOptional<float> SClothAssetSelector::GetCurrentKernelRadius() const
{
	const UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset && Asset->IsValidLod(SelectedLod))
	{
		const FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		return LodData.SkinningKernelRadius;
	}
	return TOptional<float>();
}

void SClothAssetSelector::OnCurrentKernelRadiusChanged(float InValue)
{
	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset && Asset->IsValidLod(SelectedLod))
	{
		FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		LodData.SkinningKernelRadius = InValue;
	}
}

void SClothAssetSelector::OnCurrentKernelRadiusCommitted(float InValue, ETextCommit::Type CommitType)
{
	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset && Asset->IsValidLod(SelectedLod))
	{
		FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		LodData.SkinningKernelRadius = InValue;

		// Recompute weights
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset->GetOuter()))
		{
			FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(SkeletalMesh);
			SkeletalMesh->InvalidateDeriveDataCacheGUID();
		}
	}
}

bool SClothAssetSelector::CurrentKernelRadiusIsEnabled() const
{
	return (this->GetCurrentUseMultipleInfluences() == ECheckBoxState::Checked);
}

ECheckBoxState SClothAssetSelector::GetCurrentUseMultipleInfluences() const
{
	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset && Asset->IsValidLod(SelectedLod))
	{
		const FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		return LodData.bUseMultipleInfluences ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void SClothAssetSelector::OnCurrentUseMultipleInfluencesChanged(ECheckBoxState InValue) 
{
	if (InValue == ECheckBoxState::Undetermined)
	{
		return;
	}

	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset && Asset->IsValidLod(SelectedLod))
	{
		FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		LodData.bUseMultipleInfluences = (InValue == ECheckBoxState::Checked);

		// Recompute weights
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset->GetOuter()))
		{
			FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(SkeletalMesh);
			SkeletalMesh->InvalidateDeriveDataCacheGUID();
		}
	}
}


ECheckBoxState SClothAssetSelector::GetCurrentSmoothTransition() const
{
	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset && Asset->IsValidLod(SelectedLod))
	{
		const FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		return LodData.bSmoothTransition ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}


void SClothAssetSelector::OnCurrentSmoothTransitionChanged(ECheckBoxState InValue)
{
	if (InValue == ECheckBoxState::Undetermined)
	{
		return;
	}

	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if (Asset &&Asset->IsValidLod(SelectedLod))
	{
		FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		LodData.bSmoothTransition = (InValue == ECheckBoxState::Checked);

		// Recompute weights
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset->GetOuter()))
		{
			FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(SkeletalMesh);
			SkeletalMesh->InvalidateDeriveDataCacheGUID();
		}
	}
}

bool SClothAssetSelector::IsValidClothLodSelected() const
{
	const UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	return (Asset && Asset->IsValidLod(SelectedLod));
}

TWeakObjectPtr<UClothingAssetBase> SClothAssetSelector::GetSelectedAsset() const
{
	return SelectedAsset;
	
}

int32 SClothAssetSelector::GetSelectedLod() const
{
	return SelectedLod;
}

int32 SClothAssetSelector::GetSelectedMask() const
{
	return SelectedMask;
}

void SClothAssetSelector::PostUndo(bool bSuccess)
{
	OnRefresh();
}

void SClothAssetSelector::PostRedo(bool bSuccess)
{
	OnRefresh();
}

void SClothAssetSelector::OnCopyClothingAssetSelected(const FAssetData& AssetData)
{
	if (const UObject* const SourceAsset = AssetData.GetAsset())
	{
		if (Mesh && Mesh != Cast<USkeletalMesh>(SourceAsset))
		{
			TOptional<FScopedTransaction> Transaction;

			FClothingSystemEditorInterfaceModule& ClothingEditorModule = FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>("ClothingSystemEditorInterface");
			for (const UClothingAssetFactoryBase* const ClothingAssetFactory : ClothingEditorModule.GetClothingAssetFactories())
			{
				if (SourceAsset->GetClass()->IsChildOf(ClothingAssetFactory->GetSupportedSourceAssetType()))
				{
					if (!Transaction.IsSet())
					{
						Transaction.Emplace(LOCTEXT("CreatingClothingAsset", "Creating clothing asset from another asset."));
						Mesh->Modify();
					}

					for (UClothingAssetBase* const NewAsset : ClothingAssetFactory->CreateFromSourceAsset(Mesh, SourceAsset))
					{
						Mesh->AddClothingAsset(NewAsset);
					}
				}
			}
			if (Transaction.IsSet())
			{
				OnRefresh();
			}
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

TSharedRef<SWidget> SClothAssetSelector::OnGenerateSkeletalMeshPickerForClothCopy()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FClothingSystemEditorInterfaceModule& ClothingEditorModule = FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>("ClothingSystemEditorInterface");

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SClothAssetSelector::OnCopyClothingAssetSelected);
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda(
		[this, &ClothingEditorModule](const FAssetData& AssetData)
		{
			const FSoftClassPath SoftClassPath(AssetData.AssetClassPath.ToString());
			if (UClass* const ResolvedClass = SoftClassPath.ResolveClass())
			{
				for (const UClothingAssetFactoryBase* const ClothingAssetFactory : ClothingEditorModule.GetClothingAssetFactories())
				{
					if (ResolvedClass->IsChildOf(ClothingAssetFactory->GetSupportedSourceAssetType()))
					{
						return false;
					}
				}
			}
			return true;
		});
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.bForceShowPluginContent = true;
	AssetPickerConfig.SaveSettingsName = TEXT("ClothAssetSelectorAssetPicker");
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.OnExtendAddFilterMenu = FOnExtendAddFilterMenu::CreateLambda([](UToolMenu* ToolMenu)
		{
			ToolMenu->RemoveSection("FilterBarFilterBasicAsset");  // Remove common filters section
		});
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	for (const UClothingAssetFactoryBase* const ClothingAssetFactory : ClothingEditorModule.GetClothingAssetFactories())
	{
		if (const UClass* const SupportedSourceAssetType = ClothingAssetFactory->GetSupportedSourceAssetType())
		{
			AssetPickerConfig.Filter.ClassPaths.Emplace(SupportedSourceAssetType->GetClassPathName());
		}
	}

	return SNew(SBox)
		.WidthOverride(300)
		.HeightOverride(400)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

EVisibility SClothAssetSelector::GetAssetHeaderButtonTextVisibility() const
{
	bool bShow = AssetHeaderBox.IsValid() && AssetHeaderBox->IsHovered();

	return bShow ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

EVisibility SClothAssetSelector::GetMaskHeaderButtonTextVisibility() const
{
	bool bShow = MaskHeaderBox.IsValid() && MaskHeaderBox->IsHovered();

	return bShow ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

bool SClothAssetSelector::CanSelectLod() const
{
	return Cast<const UClothingAssetCommon>(SelectedAsset.Get()) != nullptr;
}

bool SClothAssetSelector::CanEditDeformer() const
{
	return Cast<const UClothingAssetCommon>(SelectedAsset.Get()) != nullptr;
}

TSharedRef<SWidget> SClothAssetSelector::OnGetLodMenu()
{
	FMenuBuilder Builder(true, nullptr);

	int32 NumLods = 0;

	if(const UClothingAssetCommon* const CurrAsset = Cast<UClothingAssetCommon>(SelectedAsset.Get()))
	{
		NumLods = CurrAsset->GetNumLods();
	}

	if(NumLods == 0)
	{
		Builder.AddMenuEntry(LOCTEXT("LodMenu_NoLods", "Select an asset..."), FText::GetEmpty(), FSlateIcon(), FUIAction());
	}
	else
	{
		for(int32 LodIdx = 0; LodIdx < NumLods; ++LodIdx)
		{
			FText ItemText = FText::Format(LOCTEXT("LodMenuItem", "LOD{0}"), FText::AsNumber(LodIdx));
			FText ToolTipText = FText::Format(LOCTEXT("LodMenuItemToolTip", "Select LOD{0}"), FText::AsNumber(LodIdx));

			FUIAction Action;
			Action.ExecuteAction.BindSP(this, &SClothAssetSelector::OnClothingLodSelected, LodIdx);

			Builder.AddMenuEntry(ItemText, ToolTipText, FSlateIcon(), Action);
		}
	}

	return Builder.MakeWidget();
}

FText SClothAssetSelector::GetLodButtonText() const
{
	if(SelectedLod == INDEX_NONE)
	{
		return LOCTEXT("LodButtonGenTextEmpty", "LOD");
	}

	return FText::Format(LOCTEXT("LodButtonGenText", "LOD{0}"), FText::AsNumber(SelectedLod));
}

TSharedRef<ITableRow> SClothAssetSelector::OnGenerateWidgetForClothingAssetItem(TSharedPtr<FClothingAssetListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if(InItem.IsValid() && InItem->ClothingAsset.Get())
	{
		return SNew(SAssetListRow, OwnerTable, InItem)
			.OnInvalidateList(this, &SClothAssetSelector::OnRefresh);
	}

	return SNew(STableRow<TSharedPtr<FClothingAssetListItem>>, OwnerTable)
		.Content()
		[
			SNew(STextBlock).Text(FText::FromString(TEXT("No Assets Available")))
		];
}

void SClothAssetSelector::OnAssetListSelectionChanged(TSharedPtr<FClothingAssetListItem> InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	if (InSelectedItem.IsValid())
	{
		SetSelectedAsset(InSelectedItem->ClothingAsset);
	}
	else
	{
		SetSelectedAsset(nullptr);
	}
}

TSharedRef<ITableRow> SClothAssetSelector::OnGenerateWidgetForMaskItem(TSharedPtr<FClothingMaskListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if(FPointWeightMap* Mask = InItem->GetMask())
	{
		return SNew(SMaskListRow, OwnerTable, InItem, SharedThis(this))
			.OnInvalidateList(this, &SClothAssetSelector::OnRefresh);
	}

	return SNew(STableRow<TSharedPtr<FClothingMaskListItem>>, OwnerTable)
	[
		SNew(STextBlock).Text(LOCTEXT("MaskList_NoMasks", "No masks available"))
	];
}

void SClothAssetSelector::OnMaskSelectionChanged(TSharedPtr<FClothingMaskListItem> InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	if(InSelectedItem.IsValid() 
		&& InSelectedItem->ClothingAsset.IsValid() 
		&& InSelectedItem->LodIndex != INDEX_NONE 
		&& InSelectedItem->MaskIndex != INDEX_NONE
		&& InSelectedItem->MaskIndex != SelectedMask
		&& InSelectInfo != ESelectInfo::Direct)
	{
		SetSelectedMask(InSelectedItem->MaskIndex);
	}
}

FReply SClothAssetSelector::AddNewMask()
{
	if(UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get()))
	{
		if(Asset->LodData.IsValidIndex(SelectedLod))
		{
			FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
			const int32 NumRequiredValues = LodData.PhysicalMeshData.Vertices.Num();

			LodData.PointWeightMaps.AddDefaulted();

			FPointWeightMap& NewMask = LodData.PointWeightMaps.Last();

			NewMask.Name = TEXT("New Mask");
			NewMask.CurrentTarget = (uint8)EWeightMapTargetCommon::None;
			NewMask.Values.AddZeroed(NumRequiredValues);

			OnRefresh();
		}
	}

	return FReply::Handled();
}

bool SClothAssetSelector::CanAddNewMask() const
{
	return Cast<const UClothingAssetCommon>(SelectedAsset.Get()) != nullptr;
}

void SClothAssetSelector::OnRefresh()
{
	RefreshAssetList();
	RefreshMaskList();
}

void SClothAssetSelector::RefreshAssetList()
{
	UClothingAssetBase* CurrSelectedAsset = nullptr;
	int32 SelectedItem = INDEX_NONE;

	if(AssetList.IsValid())
	{
		TArray<TSharedPtr<FClothingAssetListItem>> SelectedItems;
		AssetList->GetSelectedItems(SelectedItems);

		if(SelectedItems.Num() > 0)
		{
			CurrSelectedAsset = SelectedItems[0]->ClothingAsset.Get();
		}
	}

	AssetListItems.Empty();

	for(UClothingAssetBase* Asset : Mesh->GetMeshClothingAssets())
	{
		TSharedPtr<FClothingAssetListItem> Entry = MakeShareable(new FClothingAssetListItem);

		Entry->ClothingAsset = Asset;

		AssetListItems.Add(Entry);

		if(Asset == CurrSelectedAsset)
		{
			SelectedItem = AssetListItems.Num() - 1;
		}
	}

	if(AssetListItems.Num() == 0)
	{
		// Add an invalid entry so we can show a "none" line
		AssetListItems.Add(MakeShareable(new FClothingAssetListItem));
	}

	if(AssetList.IsValid())
	{
		AssetList->RequestListRefresh();

		if(SelectedItem != INDEX_NONE)
		{
			AssetList->SetSelection(AssetListItems[SelectedItem]);
		}
		else
		{
			AssetList->ClearSelection();
		}
	}

	// Refresh LOD list
	constexpr bool bRefreshMasks = false;  // Will be done in RefreshMaskList() running afterwards
	if (const UClothingAssetCommon* const ClothingAssetCommon = Cast<UClothingAssetCommon>(CurrSelectedAsset))  // Only the UClothingAssetCommon type assets have LODs
	{
		SetSelectedLod(FMath::Min(SelectedLod, ClothingAssetCommon->GetNumLods() - 1), bRefreshMasks);
	}
	else
	{
		SetSelectedLod(INDEX_NONE, bRefreshMasks);
	}
}

void SClothAssetSelector::RefreshMaskList()
{
	int32 CurrSelectedLod = INDEX_NONE;
	int32 CurrSelectedMask = INDEX_NONE;
	int32 SelectedItem = INDEX_NONE;

	if(MaskList.IsValid())
	{
		TArray<TSharedPtr<FClothingMaskListItem>> SelectedItems;

		MaskList->GetSelectedItems(SelectedItems);

		if(SelectedItems.Num() > 0)
		{
			CurrSelectedLod = SelectedItems[0]->LodIndex;
			CurrSelectedMask = SelectedItems[0]->MaskIndex;
		}
	}

	MaskListItems.Empty();

	UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get());
	if(Asset && Asset->IsValidLod(SelectedLod))
	{
		const FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];
		const int32 NumMasks = LodData.PointWeightMaps.Num();

		for(int32 Index = 0; Index < NumMasks; ++Index)
		{
			TSharedPtr<FClothingMaskListItem> NewItem = MakeShareable(new FClothingMaskListItem);
			NewItem->ClothingAsset = Asset;
			NewItem->LodIndex = SelectedLod;
			NewItem->MaskIndex = Index;
			MaskListItems.Add(NewItem);

			if(NewItem->LodIndex == CurrSelectedLod &&
				NewItem->MaskIndex == CurrSelectedMask)
			{
				SelectedItem = MaskListItems.Num() - 1;
			}
		}
	}

	if(MaskListItems.Num() == 0)
	{
		// Add invalid entry so we can make a widget for "none"
		TSharedPtr<FClothingMaskListItem> NewItem = MakeShareable(new FClothingMaskListItem);
		MaskListItems.Add(NewItem);
	}

	if(MaskList.IsValid())
	{
		MaskList->RequestListRefresh();

		if(SelectedItem != INDEX_NONE)
		{
			MaskList->SetSelection(MaskListItems[SelectedItem]);
		}
	}
}

void SClothAssetSelector::OnClothingLodSelected(int32 InNewLod)
{
	if(InNewLod == INDEX_NONE)
	{
		SetSelectedLod(InNewLod);
		//ClothPainterSettings->OnAssetSelectionChanged.Broadcast(SelectedAsset.Get(), SelectedLod, SelectedMask);
	}

	if(const UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectedAsset.Get()))
	{
		SetSelectedLod(InNewLod);

		int32 NewMaskSelection = INDEX_NONE;
		if(Asset->LodData.IsValidIndex(SelectedLod))
		{
			const FClothLODDataCommon& LodData = Asset->LodData[SelectedLod];

			if(LodData.PointWeightMaps.Num() > 0)
			{
				NewMaskSelection = 0;
			}
		}

		SetSelectedMask(NewMaskSelection);
	}
}

void SClothAssetSelector::SetSelectedAsset(TWeakObjectPtr<UClothingAssetBase> InSelectedAsset)
{
	SelectedAsset = InSelectedAsset;

	RefreshMaskList();

	const UClothingAssetCommon* const NewAsset = Cast<const UClothingAssetCommon>(SelectedAsset.Get());
	if(NewAsset && NewAsset->GetNumLods() > 0)
	{
		SetSelectedLod(0);

		const FClothLODDataCommon& LodData = NewAsset->LodData[SelectedLod];
		if(LodData.PointWeightMaps.Num() > 0)
		{
			SetSelectedMask(0);
		}
		else
		{
			SetSelectedMask(INDEX_NONE);
		}
	}
	else
	{
		SetSelectedLod(INDEX_NONE);
		SetSelectedMask(INDEX_NONE);
	}

	OnSelectionChanged.ExecuteIfBound(SelectedAsset, SelectedLod, SelectedMask);
}

void SClothAssetSelector::SetSelectedLod(int32 InLodIndex, bool bRefreshMasks /*= true*/)
{
	if(InLodIndex != SelectedLod)
	{
		SelectedLod = InLodIndex;

		if(MaskList.IsValid() && bRefreshMasks)
		{
			// New LOD means new set of masks, refresh that list
			RefreshMaskList();
		}

		OnSelectionChanged.ExecuteIfBound(SelectedAsset, SelectedLod, SelectedMask);
	}
}

void SClothAssetSelector::SetSelectedMask(int32 InMaskIndex)
{
	SelectedMask = InMaskIndex;

	if(MaskList.IsValid())
	{
		TSharedPtr<FClothingMaskListItem>* FoundItem = nullptr;
		if(InMaskIndex != INDEX_NONE)
		{
			// Find the item so we can select it in the list
			FoundItem = MaskListItems.FindByPredicate([&](const TSharedPtr<FClothingMaskListItem>& InItem)
			{
				return InItem->MaskIndex == InMaskIndex;
			});
		}

		if(FoundItem)
		{
			MaskList->SetSelection(*FoundItem);
		}
		else
		{
			MaskList->ClearSelection();
		}
	}

	OnSelectionChanged.ExecuteIfBound(SelectedAsset, SelectedLod, SelectedMask);
}

#undef LOCTEXT_NAMESPACE
