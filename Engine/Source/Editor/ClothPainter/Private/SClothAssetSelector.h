// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class SButton;
class SHorizontalBox;
class STableViewBase;
class SWidget;
class UClothingAssetCommon;
class UClothingAssetBase;
class USkeletalMesh;
struct FAssetData;
struct FClothPhysicalMeshData;
struct FPointWeightMap;

struct FClothingAssetListItem
{
	TWeakObjectPtr<UClothingAssetBase> ClothingAsset;
};

struct FClothingMaskListItem
{
	FClothingMaskListItem()
		: LodIndex(INDEX_NONE)
		, MaskIndex(INDEX_NONE)
	{}

	FPointWeightMap* GetMask();
	FClothPhysicalMeshData* GetMeshData();
	USkeletalMesh* GetOwningMesh();

	TWeakObjectPtr<UClothingAssetCommon> ClothingAsset;
	int32 LodIndex;
	int32 MaskIndex;
};

typedef SListView<TSharedPtr<FClothingAssetListItem>> SAssetList;
typedef SListView<TSharedPtr<FClothingMaskListItem>> SMaskList;

DECLARE_DELEGATE_ThreeParams(FOnClothAssetSelectionChanged, TWeakObjectPtr<UClothingAssetBase>, int32, int32);

class SClothAssetSelector : public SCompoundWidget, public FEditorUndoClient
{
public:

	SLATE_BEGIN_ARGS(SClothAssetSelector) {}
		SLATE_EVENT(FOnClothAssetSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	~SClothAssetSelector();

	void Construct(const FArguments& InArgs, USkeletalMesh* InMesh);

	TWeakObjectPtr<UClothingAssetBase> GetSelectedAsset() const;
	int32 GetSelectedLod() const;
	int32 GetSelectedMask() const;

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	/** End FEditorUndoClient interface */

protected:

	/* Copies clothing setup from source SkelMesh */
	void OnCopyClothingAssetSelected(const FAssetData& AssetData);

	// Generate a drop-down for choosing the source skeletal mesh for copying cloth assets
	TSharedRef<SWidget> OnGenerateSkeletalMeshPickerForClothCopy();

	EVisibility GetAssetHeaderButtonTextVisibility() const;
	EVisibility GetMaskHeaderButtonTextVisibility() const;

	TSharedRef<SWidget> OnGetLodMenu();
	FText GetLodButtonText() const;

	TSharedRef<ITableRow> OnGenerateWidgetForClothingAssetItem(TSharedPtr<FClothingAssetListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnAssetListSelectionChanged(TSharedPtr<FClothingAssetListItem> InSelectedItem, ESelectInfo::Type InSelectInfo);

	TSharedRef<ITableRow> OnGenerateWidgetForMaskItem(TSharedPtr<FClothingMaskListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnMaskSelectionChanged(TSharedPtr<FClothingMaskListItem> InSelectedItem, ESelectInfo::Type InSelectInfo);

	// Mask manipulation
	FReply AddNewMask();
	bool CanAddNewMask() const;

	void OnRefresh();

	void RefreshAssetList();
	void RefreshMaskList();

	TOptional<float> GetCurrentKernelRadius() const;
	void OnCurrentKernelRadiusChanged(float InValue);
	void OnCurrentKernelRadiusCommitted(float InValue, ETextCommit::Type CommitType);
	bool CurrentKernelRadiusIsEnabled() const;

	ECheckBoxState GetCurrentUseMultipleInfluences() const;
	void OnCurrentUseMultipleInfluencesChanged(ECheckBoxState InValue);

	ECheckBoxState GetCurrentSmoothTransition() const;
	void OnCurrentSmoothTransitionChanged(ECheckBoxState InValue);

	bool IsValidClothLodSelected() const;
	
	void OnClothingLodSelected(int32 InNewLod);

	// Setters for the list selections so we can handle list selections changing properly
	void SetSelectedAsset(TWeakObjectPtr<UClothingAssetBase> InSelectedAsset);
	void SetSelectedLod(int32 InLodIndex, bool bRefreshMasks = true);
	void SetSelectedMask(int32 InMaskIndex);

	USkeletalMesh* Mesh;

	TSharedPtr<SButton> NewMaskButton;
	TSharedPtr<SAssetList> AssetList;
	TSharedPtr<SMaskList> MaskList;

	TSharedPtr<SHorizontalBox> AssetHeaderBox;
	TSharedPtr<SHorizontalBox> MaskHeaderBox;

	TArray<TSharedPtr<FClothingAssetListItem>> AssetListItems;
	TArray<TSharedPtr<FClothingMaskListItem>> MaskListItems;

	// Currently selected clothing asset, Lod Index and Mask index
	TWeakObjectPtr<UClothingAssetBase> SelectedAsset;
	int32 SelectedLod;
	int32 SelectedMask;

	FOnClothAssetSelectionChanged OnSelectionChanged;

	// Handle for mesh event callback when clothing changes.
	FDelegateHandle MeshClothingChangedHandle;

private:
	bool CanSelectLod() const;
	bool CanEditDeformer() const;
};
