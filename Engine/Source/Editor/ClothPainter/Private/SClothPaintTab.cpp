// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClothPaintTab.h"

#include "ClothPainter.h"
#include "ClothPaintingModule.h"
#include "ClothingAsset.h"
#include "ClothingPaintEditMode.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "EditorModeManager.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "IPersonaToolkit.h"
#include "ISkeletalMeshEditor.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SClothAssetSelector.h"
#include "SClothPaintWidget.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Layout/SScrollBox.h"

class UObject;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SClothPaintTab"

SClothPaintTab::SClothPaintTab() : bModeApplied(false)
{	
}

SClothPaintTab::~SClothPaintTab()
{
	if(const ISkeletalMeshEditor* Editor = static_cast<ISkeletalMeshEditor*>(HostingApp.Pin().Get()))
	{
		Editor->GetEditorModeManager().ActivateDefaultMode();
	}
}

void SClothPaintTab::Construct(const FArguments& InArgs)
{
	// Detail view for UClothingAssetCommon
	const IConsoleVariable* const EnableCardLayoutVarriable = IConsoleManager::Get().FindConsoleVariable(TEXT("DetailsPanel.Style.EnableCardLayout"));
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.StyleKey = EnableCardLayoutVarriable && EnableCardLayoutVarriable->GetBool() ?
		MakeShared<FDetailsViewStyleKey>(FDetailsViewStyleKeys::Card()) :
		MakeShared<FDetailsViewStyleKey>(FDetailsViewStyleKeys::Classic());

	DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	
	// Add delegate for editing enabled, which allows us to show a greyed out version with the CDO
	// selected when we haven't got an asset selected to avoid the UI popping.
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SClothPaintTab::IsAssetDetailsPanelEnabled));

	// Add the CDO by default
	TArray<UObject*> Objects;
	Objects.Add(UClothingAssetCommon::StaticClass()->GetDefaultObject());
	DetailsView->SetObjects(Objects, true);

	HostingApp = InArgs._InHostingApp;

	ModeWidget = nullptr;
	
	FSlateIcon TexturePaintIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.MeshPaintMode.TexturePaint");

	this->ChildSlot
	.Padding(4.f)
	[
		SAssignNew(ContentBox, SScrollBox)
	];

	
	if(ISkeletalMeshEditor* SkeletalMeshEditor = GetSkeletalMeshEditor())
	{
		IPersonaToolkit& Persona = SkeletalMeshEditor->GetPersonaToolkit().Get();

		ContentBox->AddSlot()
		[
			SAssignNew(SelectorWidget, SClothAssetSelector, Persona.GetMesh())
				.OnSelectionChanged(this, &SClothPaintTab::OnAssetSelectionChanged)
		];

		ContentBox->AddSlot()
		[
			DetailsView->AsShared()
		];
	}
}

void SClothPaintTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SClothPaintTab::EnterPaintMode()
{
	const ISkeletalMeshEditor* SkeletalMeshEditor = GetSkeletalMeshEditor();
	if (!SkeletalMeshEditor)
	{
		return;
	}
	FClothingPaintEditMode* PaintMode = (FClothingPaintEditMode*)SkeletalMeshEditor->GetEditorModeManager().GetActiveMode(PaintModeID);
	if (!PaintMode)
	{
		return;
	}
	
	FClothPainter* ClothPainter = static_cast<FClothPainter*>(PaintMode->GetMeshPainter());
	check(ClothPainter);

	ClothPainter->Reset();
	ModeWidget = StaticCastSharedPtr<SClothPaintWidget>(ClothPainter->GetWidget());
	
	ContentBox->AddSlot()
	[
		ModeWidget->AsShared()
	];

	if(SelectorWidget.IsValid())
	{
		UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(SelectorWidget->GetSelectedAsset().Get());
		ClothPainter->OnAssetSelectionChanged(Asset, SelectorWidget->GetSelectedLod(), SelectorWidget->GetSelectedMask());
	}
}

void SClothPaintTab::ExitPaintMode()
{
	ContentBox->RemoveSlot(ModeWidget->AsShared());
	ModeWidget = nullptr;
}

void SClothPaintTab::OnAssetSelectionChanged(TWeakObjectPtr<UClothingAssetBase> InAssetPtr, int32 InLodIndex, int32 InMaskIndex)
{
	const ISkeletalMeshEditor* SkeletalMeshEditor = GetSkeletalMeshEditor();
	if (!SkeletalMeshEditor)
	{
		return;
	}

	FClothingPaintEditMode* PaintMode = (FClothingPaintEditMode*)SkeletalMeshEditor->GetEditorModeManager().GetActiveMode(PaintModeID);
	if(PaintMode)
	{
		if(FClothPainter* const ClothPainter = static_cast<FClothPainter*>(PaintMode->GetMeshPainter()))
		{
			ClothPainter->OnAssetSelectionChanged(Cast<UClothingAssetCommon>(InAssetPtr.Get()), InLodIndex, InMaskIndex);
		}
	}

	TArray<UObject*> Objects;
	if(UClothingAssetBase* const Asset = InAssetPtr.Get())
	{
		Objects.Add(Asset);
	}
	else
	{
		Objects.Add(UClothingAssetCommon::StaticClass()->GetDefaultObject());  // Add CDO by default
	}
	DetailsView->SetObjects(Objects, true);
}

bool SClothPaintTab::IsAssetDetailsPanelEnabled()
{
	// Only enable editing if we have a valid details panel that is not observing the CDO
	if(DetailsView.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailsView->GetSelectedObjects();

		if(SelectedObjects.Num() > 0)
		{
			return SelectedObjects[0].Get() != UClothingAssetCommon::StaticClass()->GetDefaultObject();
		}
	}

	return false;
}

TSharedRef<IPersonaToolkit> SClothPaintTab::GetPersonaToolkit() const
{
	return GetSkeletalMeshEditor()->GetPersonaToolkit();
}

ISkeletalMeshEditor* SClothPaintTab::GetSkeletalMeshEditor() const
{
	ISkeletalMeshEditor* Editor = static_cast<ISkeletalMeshEditor*>(HostingApp.Pin().Get());
	check(Editor);

	return Editor;
}

#undef LOCTEXT_NAMESPACE //"SClothPaintTab"
