// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CLOTHPAINTER_API

class IDetailsView;
class IPersonaToolkit;
class ISkeletalMeshEditor;
class SClothAssetSelector;
class SClothPaintWidget;
class SScrollBox;
class UClothingAssetBase;
struct FGeometry;

class SClothPaintTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SClothPaintTab)
	{}	
	SLATE_ARGUMENT(TWeakPtr<class FAssetEditorToolkit>, InHostingApp)
	SLATE_END_ARGS()

	UE_API SClothPaintTab();
	UE_API ~SClothPaintTab();

	/** SWidget functions */
	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Setup and teardown the cloth paint UI */
	UE_API void EnterPaintMode();
	UE_API void ExitPaintMode();

protected:

	/** Called from the selector when the asset selection changes (Asset, LOD, Mask) */
	UE_API void OnAssetSelectionChanged(TWeakObjectPtr<UClothingAssetBase> InAssetPtr, int32 InLodIndex, int32 InMaskIndex);

	/** Whether or not the asset config section is enabled for editing */
	UE_API bool IsAssetDetailsPanelEnabled();

	/** Helpers for getting editor objects */
	UE_API ISkeletalMeshEditor* GetSkeletalMeshEditor() const;
	UE_API TSharedRef<IPersonaToolkit> GetPersonaToolkit() const;

	TWeakPtr<class FAssetEditorToolkit> HostingApp;
	
	TSharedPtr<SClothAssetSelector> SelectorWidget;
	TSharedPtr<SClothPaintWidget> ModeWidget;
	TSharedPtr<SScrollBox> ContentBox;
	TSharedPtr<IDetailsView> DetailsView;

	bool bModeApplied;
};

#undef UE_API
