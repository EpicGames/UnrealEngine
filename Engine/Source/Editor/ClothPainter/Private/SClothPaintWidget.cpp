// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClothPaintWidget.h"

#include "ClothPaintSettings.h"
#include "ClothPaintSettingsCustomization.h"
#include "ClothPaintToolBase.h"
#include "ClothPainter.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "MeshPaintSettings.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"

class UObject;

#define LOCTEXT_NAMESPACE "ClothPaintWidget"

void SClothPaintWidget::Construct(const FArguments& InArgs, FClothPainter* InPainter)
{
	Painter = InPainter;

	if(Painter)
	{
		Objects.Add(Painter->GetBrushSettings());
		Objects.Add(Painter->GetPainterSettings());

		UObject* ToolSettings = Painter->GetSelectedTool()->GetSettingsObject();
		if(ToolSettings)
		{
			Objects.Add(ToolSettings);
			Painter->GetSelectedTool()->RegisterSettingsObjectCustomizations(DetailsView.Get());
		}

		ClothPainterSettings = Cast<UClothPainterSettings>(InPainter->GetPainterSettings());
		CreateDetailsView(InPainter);
	}

	ChildSlot
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		.Padding(FMargin(0.0f, 3.0f, 0.0f, 0.0f))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0)
				[
					DetailsView->AsShared()
				]
			]
		]
	];
}

void SClothPaintWidget::CreateDetailsView(FClothPainter* InPainter)
{
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	
	DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(UClothPainterSettings::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FClothPaintSettingsCustomization::MakeInstance, InPainter));
	DetailsView->RegisterInstancedCustomPropertyLayout(UPaintBrushSettings::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FClothPaintBrushSettingsCustomization::MakeInstance));

	DetailsView->SetObjects(Objects, true);
}

void SClothPaintWidget::OnRefresh()
{
	if(DetailsView.IsValid())
	{
		Objects.Reset();

		Objects.Add(Painter->GetPainterSettings());

		UObject* ToolSettings = Painter->GetSelectedTool()->GetSettingsObject();
		if(ToolSettings)
		{
			Objects.Add(ToolSettings);
			Painter->GetSelectedTool()->RegisterSettingsObjectCustomizations(DetailsView.Get());
		}

		Objects.Add(Painter->GetBrushSettings());

		DetailsView->SetObjects(Objects, true);
	}
}

void SClothPaintWidget::Reset()
{
	OnRefresh();
}

#undef LOCTEXT_NAMESPACE
