// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FClothPainter;
class IDetailsView;
class UClothPainterSettings;
class UObject;

class SClothPaintWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SClothPaintWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FClothPainter* InPainter);
	void CreateDetailsView(FClothPainter* InPainter);

	// Refresh the widget such as when entering the paint mode
	void OnRefresh();

	// Resets the selections and puts the widget back to starting state
	void Reset();

protected:

	// Details view placed below asset selection
	TSharedPtr<IDetailsView> DetailsView;

	// Objects observed in the details view
	TArray<UObject*> Objects;

	// The painter instance this widget is using
	FClothPainter* Painter;

	// Settings for the painter instance
	UClothPainterSettings* ClothPainterSettings;
};
