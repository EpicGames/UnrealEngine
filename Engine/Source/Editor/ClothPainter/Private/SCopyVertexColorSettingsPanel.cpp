// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCopyVertexColorSettingsPanel.h"

#include "ClothLODData.h"
#include "ClothPhysicalMeshData.h"
#include "ClothingAsset.h"
#include "Containers/Array.h"
#include "DetailsViewArgs.h"
#include "Framework/Application/SlateApplication.h"
#include "IStructureDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PointWeightMap.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "CopyVertexColorSettings"

void SCopyVertexColorSettingsPanel::Construct(const FArguments& InArgs, UClothingAssetCommon* InAsset, int32 InLOD, FPointWeightMap* InMask)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedPtr<IStructureDetailsView> StructureDetailsView;

	SelectedAssetPtr = InAsset;
	SelectedLOD = InLOD;
	SelectedMask = InMask;

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = nullptr;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

	FStructOnScope* Struct = new FStructOnScope(FCopyVertexColorToClothParams::StaticStruct(), (uint8*)&CopyParams);
	StructureDetailsView->SetStructureData(MakeShareable(Struct));

	this->ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(300.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.MaxHeight(500.0f)
			.Padding(2)
			[
				StructureDetailsView->GetWidget()->AsShared()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)

				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Label_Copy", "Copy"))
					.OnClicked(this, &SCopyVertexColorSettingsPanel::OnCopyClicked)
					.ToolTipText(LOCTEXT("Label_Copy_Tooltip", "Copy vertex colors from selected channel to this mask."))
				]
			]
		]
	];
}

/** Util for converting one channel of an FColor to a float */
float GetColorChannelAsFloat(const FColor& Color, ESourceColorChannel Channel, float ScalingFactor)
{
	uint8 Value = 0;
	switch (Channel)
	{
	case ESourceColorChannel::Red:
		Value = Color.R;
		break;
	case ESourceColorChannel::Green:
		Value = Color.G;
		break;
	case ESourceColorChannel::Blue:
		Value = Color.B;
		break;
	case ESourceColorChannel::Alpha:
		Value = Color.A;
		break;
	}

	return (Value / 255.f) * ScalingFactor;
}

/** Copy channel of vertex colors into a particular mask entry */
FReply SCopyVertexColorSettingsPanel::OnCopyClicked()
{
	UClothingAssetCommon* Asset = SelectedAssetPtr.Get();
	if (Asset && SelectedMask)
	{
		const FClothLODDataCommon& ClothLODData = Asset->LodData[SelectedLOD];

		check(ClothLODData.PhysicalMeshData.Vertices.Num() == ClothLODData.PhysicalMeshData.VertexColors.Num());
		int32 NumVerts = ClothLODData.PhysicalMeshData.Vertices.Num();
		check(SelectedMask->Values.Num() == NumVerts);

		for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
		{
			const FColor VertColor = ClothLODData.PhysicalMeshData.VertexColors[VertIdx];
			SelectedMask->Values[VertIdx] = GetColorChannelAsFloat(VertColor, CopyParams.ColorChannel, CopyParams.ScalingFactor);
		}
	}

	// Close the menu we created
	FSlateApplication::Get().DismissAllMenus();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
