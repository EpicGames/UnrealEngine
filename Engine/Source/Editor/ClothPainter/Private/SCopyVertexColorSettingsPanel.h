// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CopyVertexColorToClothParams.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UClothingAssetCommon;
struct FPointWeightMap;

/** Widget used for copying vertex colors from sim mesh to a selected mask. */
class SCopyVertexColorSettingsPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCopyVertexColorSettingsPanel)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UClothingAssetCommon* InAsset, int32 InLOD, FPointWeightMap* InMask);

private:

	// Params struct, displayed using details panel
	FCopyVertexColorToClothParams CopyParams;

	// Handle 'Copy' button being clicked
	FReply OnCopyClicked();

	// Pointer to currently selected ClothingAsset
	TWeakObjectPtr<UClothingAssetCommon> SelectedAssetPtr;
	// Pointer to selected mask
	FPointWeightMap* SelectedMask;
	// Currently selected LOD
	int32 SelectedLOD;
};