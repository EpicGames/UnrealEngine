// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Internationalization/Text.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class FApplicationMode;
class FExtender;
class FUICommandList;
class ISkeletalMeshEditor;
class SClothPaintTab;
class UEditorExperimentalSettings;

CLOTHPAINTER_API extern const FName PaintModeID;

class FClothPaintingModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// Setup and register our editmode
	void SetupMode();
	
	// Unregister and shut down our edit mode
	void ShutdownMode();

protected:

	// Extends the skeletal mesh editor mode
	TSharedRef<FApplicationMode> ExtendApplicationMode(const FName ModeName, TSharedRef<FApplicationMode> InMode);
protected:

	TArray<TWeakPtr<FApplicationMode>> RegisteredApplicationModes;
	FWorkflowApplicationModeExtender Extender;

private:

	// Extend toolbars and menus
	void RegisterMenus();

	// Extends a skeletal mesh editor instance toolbar
	TSharedRef<FExtender> ExtendSkelMeshEditorToolbar(const TSharedRef<FUICommandList> InCommandList, TSharedRef<ISkeletalMeshEditor> InSkeletalMeshEditor);

	// Handle for the extender delegate we created
	FDelegateHandle SkelMeshEditorExtenderHandle;

	// Gets text for the enable paint tools button
	FText GetPaintToolsButtonText(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const;

	// Gets tool tip for the enable paint tools button
	FText GetPaintToolsButtonToolTip(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const;

	// Toggles paint mode on the clothing tab
	void OnToggleMode(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const;

	// Return true if currently in paint mode
	bool IsPaintModeActive(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const;

	// Gets the current active clothing tab, will invoke (spawn or draw attention to) if bInvoke == true
	TSharedPtr<SClothPaintTab> GetActiveClothTab(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor, bool bInvoke = true) const;
};
