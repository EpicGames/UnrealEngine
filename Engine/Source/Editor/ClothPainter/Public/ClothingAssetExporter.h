// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothingAssetExporter.generated.h"

class UClass;
class UClothingAssetBase;
class UClothingAssetExporter;
template <typename FuncType> class TFunctionRef;

/** A modular interface to provide ways to export clothing assets. */
class IClothingAssetExporterClassProvider : public IModularFeature
{
public:
	inline static const FName FeatureName = TEXT("ClothingAssetExporterClassProvider");

	virtual TSubclassOf<UClothingAssetExporter> GetClothingAssetExporterClass() const = 0;
};

/** Modular exporter base class. */
UCLASS(MinimalAPI, Abstract)
class UClothingAssetExporter : public UObject
{
	GENERATED_BODY()

public:
	/** Return the class of the exported asset type. */
	virtual UClass* GetExportedType() const
	PURE_VIRTUAL(UClothingAssetExporter::GetExportedType, return nullptr;);

	/**
	 * Export the specified asset.
	 * \param ClothingAsset the source clothing asset to be exported.
	 * \param ExportedAsset the destination asset object in the exported type provided by the caller, ready to be filled by the Export function.
	 */
	virtual void Export(const UClothingAssetBase* ClothingAsset, UObject* ExportedAsset)
	PURE_VIRTUAL(UClothingAssetExporter::Export,);
};

/**
 * Call the specified function for each asset exporter plugin currently loaded, passing the target exported type as a parameter.
 * \param Function the function to call for each asset exporter found.
 */
void CLOTHPAINTER_API ForEachClothingAssetExporter(TFunctionRef<void(UClass*)> Function);

/**
 * Open a dialog and create a new asset of the specified type, then call the associated export function on it.
 * \param ClothingAsset the asset to export.
 * \param ExportedType the exported type.
 */
void CLOTHPAINTER_API ExportClothingAsset(const UClothingAssetBase* ClothingAsset, UClass* ExportedType);
