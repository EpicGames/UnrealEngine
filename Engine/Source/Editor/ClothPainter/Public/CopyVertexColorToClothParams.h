// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "CopyVertexColorToClothParams.generated.h"

/** Enum for selecting color channel to copy */
UENUM()
enum class ESourceColorChannel : uint8
{
	Red,
	Green,
	Blue,
	Alpha
};

/** Helper struct used for specifying options when copying vertex colors */
USTRUCT()
struct FCopyVertexColorToClothParams
{
	GENERATED_BODY()

	FCopyVertexColorToClothParams()
		: ColorChannel(ESourceColorChannel::Red)
		, ScalingFactor(1.f)
	{}

	/** Color channel to copy from vertex colors. */
	UPROPERTY(EditAnywhere, Category="Copy Vertex Colors")
	ESourceColorChannel ColorChannel;

	/** Scaling factor applied to vertex colours (in range 0-1) before applying to mask.  */
	UPROPERTY(EditAnywhere, Category = "Copy Vertex Colors")
	float ScalingFactor;
};